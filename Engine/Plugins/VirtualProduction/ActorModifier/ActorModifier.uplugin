{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Actor Modifier",
	"Description": "Actual implementation of modifiers for actors based on ActorModifierCore plugin",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"PlatformAllowList": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "ActorModifier",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorModifierEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorModifierLayout",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ActorModifierRendering",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ActorModifierCore",
			"Enabled": true
		},
		{
			"Name": "ClonerEffector",
			"Enabled": true
		},
		{
			"Name": "CompositeCore",
			"Enabled": true
		}
	]
}