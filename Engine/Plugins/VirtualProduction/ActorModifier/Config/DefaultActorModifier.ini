[CoreRedirects]
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaTransformSharedModifierState",NewName="/Script/ActorModifier.EActorModifierTransformSharedState")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaReferenceContainer",NewName="/Script/ActorModifier.EActorModifierReferenceContainer")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaVisibilityActor",NewName="/Script/ActorModifier.EActorModifierVisibilityActor")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaCorner2D",NewName="/Script/ActorModifierLayout.EActorModifierGridArrangeCorner2D")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaAxis",NewName="/Script/ActorModifier.EActorModifierAlignment")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaGridArrangeDirection",NewName="/Script/ActorModifierLayout.EActorModifierGridArrangeDirection")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaJustifyVertical",NewName="/Script/ActorModifierLayout.EActorModifierJustifyVertical")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaJustifyHorizontal",NewName="/Script/ActorModifierLayout.EActorModifierJustifyHorizontal")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaJustifyDepth",NewName="/Script/ActorModifierLayout.EActorModifierJustifyDepth")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaRadialArrangement",NewName="/Script/ActorModifierLayout.EActorModifierRadialArrangeMode")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaRadialArrangePlane",NewName="/Script/ActorModifierLayout.EActorModifierRadialArrangePlane")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaModifiersAxis",NewName="/Script/ActorModifier.EActorModifierAxis")

+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaTransformSharedModifierState",NewName="/Script/ActorModifier.ActorModifierTransformSharedModifierState")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaTransformSharedActorState",NewName="/Script/ActorModifier.ActorModifierTransformSharedActorState")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaSceneTreeActor",NewName="/Script/ActorModifier.ActorModifierSceneTreeActor")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaVisibilitySharedModifierState",NewName="/Script/ActorModifier.ActorModifierVisibilitySharedModifierState")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaVisibilitySharedActorState",NewName="/Script/ActorModifier.ActorModifierVisibilitySharedActorState")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaAlignBetweenWeightedActor",NewName="/Script/ActorModifierLayout.ActorModifierAlignBetweenWeightedActor")
+StructRedirects=(OldName="/Script/Avalanche.AvaAnchorAlignment",NewName="/Script/ActorModifier.ActorModifierAnchorAlignment")

+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaTransformModifierShared",NewName="/Script/ActorModifier.ActorModifierTransformShared")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaVisibilityModifierShared",NewName="/Script/ActorModifier.ActorModifierVisibilityShared")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaAttachmentBaseModifier",NewName="/Script/ActorModifier.ActorModifierAttachmentBaseModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaArrangeBaseModifier",NewName="/Script/ActorModifier.ActorModifierArrangeBaseModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaAlignBetweenModifier",NewName="/Script/ActorModifierLayout.ActorModifierAlignBetweenModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaAutoFollowModifier",NewName="/Script/ActorModifierLayout.ActorModifierAutoFollowModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaGridArrangeModifier",NewName="/Script/ActorModifierLayout.ActorModifierGridArrangeModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaJustifyModifier",NewName="/Script/ActorModifierLayout.ActorModifierJustifyModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaLookAtModifier",NewName="/Script/ActorModifierLayout.ActorModifierLookAtModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaRadialArrangeModifier",NewName="/Script/ActorModifierLayout.ActorModifierRadialArrangeModifier")
+ClassRedirects=(OldName="/Script/ActorModifierLayout.ActorModifierLayoutSplinePathModifier",NewName="/Script/ActorModifierLayout.ActorModifierSplinePathModifier")
