// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierTypes.h"

FVector FActorModifierAnchorAlignment::LocalBoundsOffset(const FBox& InBounds, const bool bInInverted) const
{
	const FVector Extent = InBounds.GetExtent();

	FVector BoundsOffset = FVector::ZeroVector;

	auto HorizontalAlign = [bInInverted, Extent]() -> double { return bInInverted ? Extent.Y : -Extent.Y; };
	auto VerticalAlign = [bInInverted, Extent]() -> double { return bInInverted ? Extent.Z : -Extent.Z; };
	auto DepthAlign = [bInInverted, Extent]() -> double { return bInInverted ? Extent.X : -Extent.X; };

	switch (Horizontal)
	{
	case EActorModifierHorizontalAlignment::Left: BoundsOffset.Y = HorizontalAlign(); break;
	case EActorModifierHorizontalAlignment::Center: break;
	case EActorModifierHorizontalAlignment::Right: BoundsOffset.Y = -HorizontalAlign(); break;
	}

	switch (Vertical)
	{
	case EActorModifierVerticalAlignment::Top: BoundsOffset.Z = VerticalAlign(); break;
	case EActorModifierVerticalAlignment::Center: break;
	case EActorModifierVerticalAlignment::Bottom: BoundsOffset.Z = -VerticalAlign(); break;
	}

	switch (Depth)
	{
	case EActorModifierDepthAlignment::Front: BoundsOffset.X = DepthAlign(); break;
	case EActorModifierDepthAlignment::Center: break;
	case EActorModifierDepthAlignment::Back: BoundsOffset.X = -DepthAlign(); break;
	}

	return BoundsOffset;
}
