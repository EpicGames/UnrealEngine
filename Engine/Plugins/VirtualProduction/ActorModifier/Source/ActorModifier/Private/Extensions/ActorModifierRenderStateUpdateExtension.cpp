// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/ActorModifierRenderStateUpdateExtension.h"

#include "Components/PrimitiveComponent.h"
#include "Containers/Ticker.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Utilities/ActorModifierActorUtils.h"

FActorModifierRenderStateUpdateExtension::FActorModifierRenderStateUpdateExtension(TNotNull<IActorModifierRenderStateUpdateHandler*> InExtensionHandler)
	: ExtensionHandlerWeak(InExtensionHandler)
	, RelevantReasons(InExtensionHandler->GetRenderStateDirtyEventTypes())
{
}

void FActorModifierRenderStateUpdateExtension::TrackActorVisibility(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	if (IsActorVisibilityTracked(InActor))
	{
		return;
	}

	TrackedActorsVisibility.Add(InActor, UE::ActorModifier::ActorUtils::IsActorVisible(InActor));
}

void FActorModifierRenderStateUpdateExtension::UntrackActorVisibility(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	TrackedActorsVisibility.Remove(InActor);
}

bool FActorModifierRenderStateUpdateExtension::IsActorVisibilityTracked(AActor* InActor) const
{
	return InActor && TrackedActorsVisibility.Contains(InActor);
}

void FActorModifierRenderStateUpdateExtension::SetTrackedActorsVisibility(const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	// Remove unwanted actors, keep value for tracked ones
	for (TMap<TWeakObjectPtr<AActor>, bool>::TIterator It(TrackedActorsVisibility); It; ++It)
	{
		if (!InActors.Contains(It->Key))
		{
			It.RemoveCurrent();
		}
	}

	// Track value of wanted actors, will not overwrite already tracked one
	for (const TWeakObjectPtr<AActor>& Actor : InActors)
	{
		TrackActorVisibility(Actor.Get());
	}
}

void FActorModifierRenderStateUpdateExtension::SetTrackedActorVisibility(AActor* InActor, bool bInIncludeChildren)
{
	if (!InActor)
	{
		return;
	}

	TSet<TWeakObjectPtr<AActor>> TrackActors {InActor};

	if (bInIncludeChildren)
	{
		TArray<AActor*> AttachedActors;
		InActor->GetAttachedActors(AttachedActors, false, true);
		Algo::Transform(AttachedActors, TrackActors, [](AActor* InAttachedActor)
		{
			return InAttachedActor;
		});
	}

	SetTrackedActorsVisibility(TrackActors);
}

void FActorModifierRenderStateUpdateExtension::OnExtensionEnabled(EActorModifierCoreEnableReason InReason)
{
	BindDelegate();
}

void FActorModifierRenderStateUpdateExtension::OnExtensionDisabled(EActorModifierCoreDisableReason InReason)
{
	UnbindDelegate();
}

void FActorModifierRenderStateUpdateExtension::OnRenderStateDirty(UActorComponent& InComponent)
{
	if (!UE::ActorModifierCore::IsRenderStateDirtyRelevant(RelevantReasons))
	{
		return;
	}

	const AActor* ModifierActor = GetModifierActor();
	AActor* ActorDirty = InComponent.GetOwner();

	if (!ActorDirty || !ModifierActor)
	{
		return;
	}

	if (ActorDirty->GetLevel() != ModifierActor->GetLevel())
	{
		return;
	}

	const UActorModifierCoreBase* Modifier = GetModifier();
	if (!Modifier || !Modifier->IsModifierEnabled() || !Modifier->IsModifierIdle())
	{
		return;
	}

	if (IActorModifierRenderStateUpdateHandler* HandlerInterface = ExtensionHandlerWeak.Get())
	{
		HandlerInterface->OnRenderStateUpdated(ActorDirty, &InComponent);
	}

	// Skip if render state is not relevant or not tracking the actor
	if (!UE::ActorModifierCore::IsRenderStateDirtyRelevant(UE::ActorModifierCore::ERenderStateDirtyReason::Visibility)
		|| !TrackedActorsVisibility.Contains(ActorDirty))
	{
		return;
	}

	// Execute on next tick otherwise visibility data might not be up to date
	TWeakObjectPtr<AActor> ActorDirtyWeak(ActorDirty);
	UActorModifierCoreSubsystem::GetModifierTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this, ActorDirtyWeak](float InDeltaSeconds)->bool
	{
		AActor* ActorDirty = ActorDirtyWeak.Get();

		if (!ActorDirty)
		{
			return false;
		}

		bool* bOldVisibility = TrackedActorsVisibility.Find(ActorDirty);

		if (!bOldVisibility)
		{
			return false;
		}

		const bool bNewVisibility = UE::ActorModifier::ActorUtils::IsActorVisible(ActorDirty);
		bool& bOldVisibilityRef = *bOldVisibility;

		const bool bVisibilityChanged = bOldVisibilityRef != bNewVisibility;

		// Set new visibility
		bOldVisibilityRef = bNewVisibility;

		IActorModifierRenderStateUpdateHandler* HandlerInterface = ExtensionHandlerWeak.Get();
		if (bVisibilityChanged && HandlerInterface)
		{
			HandlerInterface->OnActorVisibilityChanged(ActorDirty);
		}

		return false;
	}));
}

void FActorModifierRenderStateUpdateExtension::BindDelegate()
{
	UActorComponent::MarkRenderStateDirtyEvent.RemoveAll(this);
	UActorComponent::MarkRenderStateDirtyEvent.AddSP(this, &FActorModifierRenderStateUpdateExtension::OnRenderStateDirty);
}

void FActorModifierRenderStateUpdateExtension::UnbindDelegate()
{
	UActorComponent::MarkRenderStateDirtyEvent.RemoveAll(this);
}
