// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"

#include "Containers/Ticker.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Utilities/ActorModifierActorUtils.h"

FActorModifierSceneTreeUpdateExtension::FOnGetSceneTreeResolver FActorModifierSceneTreeUpdateExtension::OnGetSceneTreeResolverDelegate;

FActorModifierSceneTreeUpdateExtension::FActorModifierSceneTreeUpdateExtension(IActorModifierSceneTreeUpdateHandler* InExtensionHandler)
	: ExtensionHandlerWeak(InExtensionHandler)
{
	check(InExtensionHandler);
}

void FActorModifierSceneTreeUpdateExtension::TrackSceneTree(int32 InTrackedActorIdx, FActorModifierSceneTreeActor* InTrackedActor)
{
	if (!InTrackedActor)
	{
		return;
	}

	InTrackedActor->LocalActorWeak = GetModifierActor();
	InTrackedActor->ReferenceActorsWeak.Empty();
	InTrackedActor->ReferenceActorChildrenWeak.Empty();
	InTrackedActor->ReferenceActorParentsWeak.Empty();
	InTrackedActor->ReferenceActorDirectChildrenWeak.Empty();

	TrackedActors.Add(InTrackedActorIdx, InTrackedActor);

	CheckTrackedActorUpdate(InTrackedActorIdx);
}

void FActorModifierSceneTreeUpdateExtension::UntrackSceneTree(int32 InTrackedActorIdx)
{
	if (!TrackedActors.Contains(InTrackedActorIdx))
	{
		return;
	}

	TrackedActors.Remove(InTrackedActorIdx);
}

FActorModifierSceneTreeActor* FActorModifierSceneTreeUpdateExtension::GetTrackedActor(int32 InTrackedActorIdx) const
{
	if (FActorModifierSceneTreeActor* const* TrackedActor = TrackedActors.Find(InTrackedActorIdx))
	{
		return *TrackedActor;
	}

	return nullptr;
}

void FActorModifierSceneTreeUpdateExtension::CheckTrackedActorsUpdate() const
{
	// Container could change while in range iteration
	TArray<int32> TrackedKeys;
	TrackedActors.GenerateKeyArray(TrackedKeys);

	for (const int32 Key : TrackedKeys)
	{
		CheckTrackedActorUpdate(Key);
	}
}

void FActorModifierSceneTreeUpdateExtension::OnExtensionInitialized()
{
	ULevel* Level = GetModifierLevel();
	if (OnGetSceneTreeResolverDelegate.IsBound() && IsValid(Level))
	{
		SceneTreeResolver = OnGetSceneTreeResolverDelegate.Execute(Level);
	}
}

void FActorModifierSceneTreeUpdateExtension::OnExtensionEnabled(EActorModifierCoreEnableReason InReason)
{
	const UWorld* World = GetModifierWorld();
	if (!IsValid(World))
	{
		return;
	}

	// When actor are destroyed in world
	World->RemoveOnActorDestroyedHandler(WorldActorDestroyedDelegate);
	WorldActorDestroyedDelegate = World->AddOnActorDestroyedHandler(FOnActorDestroyed::FDelegate::CreateSP(this, &FActorModifierSceneTreeUpdateExtension::OnWorldActorDestroyed));

	// Used to detect visibility changes in siblings
	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);
	USceneComponent::MarkRenderStateDirtyEvent.AddSP(this, &FActorModifierSceneTreeUpdateExtension::OnRenderStateDirty);

	if (SceneTreeResolver.IsValid())
	{
		SceneTreeResolver->OnActorHierarchyChanged().AddSP(this, &FActorModifierSceneTreeUpdateExtension::OnRefreshTrackedActors);
		SceneTreeResolver->Activate();
	}
	
	CheckTrackedActorsUpdate();
}

void FActorModifierSceneTreeUpdateExtension::OnExtensionDisabled(EActorModifierCoreDisableReason InReason)
{
	USceneComponent::MarkRenderStateDirtyEvent.RemoveAll(this);

	const UWorld* World = GetModifierWorld();
	if (!IsValid(World))
	{
		return;
	}

	World->RemoveOnActorDestroyedHandler(WorldActorDestroyedDelegate);

	if (SceneTreeResolver.IsValid())
	{
		SceneTreeResolver->OnActorHierarchyChanged().RemoveAll(this);
		SceneTreeResolver->Deactivate();
	}
}

void FActorModifierSceneTreeUpdateExtension::OnRefreshTrackedActors(AActor* InActor)
{
	// Container could change while in range iteration
	TArray<int32> TrackedKeys;
	TrackedActors.GenerateKeyArray(TrackedKeys);

	for (const int32 Key : TrackedKeys)
	{
		CheckTrackedActorUpdate(Key);

		if (InActor)
		{
			if (const FActorModifierSceneTreeActor* TrackedActor = GetTrackedActor(Key))
			{
				const AActor* ReferenceActor = TrackedActor->ReferenceActorWeak.Get();
				IActorModifierSceneTreeUpdateHandler* HandlerInterface = ExtensionHandlerWeak.Get();

				if (HandlerInterface
					&& IsValid(ReferenceActor)
					&& TrackedActor->ReferenceContainer == EActorModifierReferenceContainer::Other
					&& (InActor == ReferenceActor || InActor->IsAttachedTo(ReferenceActor)))
				{
					HandlerInterface->OnSceneTreeTrackedActorRearranged(Key, InActor);
				}
			}
		}
	}
}

void FActorModifierSceneTreeUpdateExtension::OnRenderStateDirty(UActorComponent& InComponent)
{
	// Scene tree update is only concerned with scene tree / visibility changes
	constexpr UE::ActorModifierCore::ERenderStateDirtyReason RelevantReasons = UE::ActorModifierCore::ERenderStateDirtyReason::Visibility 
		| UE::ActorModifierCore::ERenderStateDirtyReason::SceneTree;

	if (!UE::ActorModifierCore::IsRenderStateDirtyRelevant(RelevantReasons))
	{
		return;
	}

	AActor* OwningActor = InComponent.GetOwner();
	if (!IsValid(OwningActor))
	{
		return;
	}

	if (GetModifierWorld() != OwningActor->GetWorld())
	{
		return;
	}

	// Container could change while in range iteration
	TArray<int32> TrackedKeys;
	TrackedActors.GenerateKeyArray(TrackedKeys);

	for (const int32 Key : TrackedKeys)
	{
		const FActorModifierSceneTreeActor* TrackedActor = GetTrackedActor(Key);
		if (!TrackedActor || !TrackedActor->LocalActorWeak.IsValid())
		{
			continue;
		}

		const AActor* ReferenceActor = TrackedActor->ReferenceActorWeak.Get();

		const bool bIsReferenceActor = ReferenceActor == OwningActor || TrackedActor->ReferenceActorsWeak.Contains(OwningActor);
		const bool bIsReferenceActorChild = ReferenceActor && OwningActor->IsAttachedTo(ReferenceActor);
		const bool bIsReferenceActorSibling = ReferenceActor && OwningActor->GetAttachParentActor() == ReferenceActor->GetAttachParentActor();
		const bool bIsReferenceActorParent = ReferenceActor && ReferenceActor->IsAttachedTo(OwningActor);

		if (!bIsReferenceActor && !bIsReferenceActorChild && !bIsReferenceActorSibling && !bIsReferenceActorParent)
		{
			continue;
		}

		CheckTrackedActorUpdate(Key);
	}
}

void FActorModifierSceneTreeUpdateExtension::OnWorldActorDestroyed(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	// Delay check from one tick to make sure actor is no longer attached
	UActorModifierCoreSubsystem::GetModifierTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float InDeltaSeconds)->bool
	{
		CheckTrackedActorsUpdate();
		return false;
	}));
}

bool FActorModifierSceneTreeUpdateExtension::IsSameActorArray(const TArray<TWeakObjectPtr<AActor>>& InPreviousActorWeak, const TArray<TWeakObjectPtr<AActor>>& InNewActorWeak) const
{
	if (InPreviousActorWeak.Num() != InNewActorWeak.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < InPreviousActorWeak.Num(); Idx++)
	{
		if (InPreviousActorWeak[Idx].Get() != InNewActorWeak[Idx].Get())
		{
			return false;
		}
	}

	return true;
}

TSet<TWeakObjectPtr<AActor>> FActorModifierSceneTreeUpdateExtension::GetChildrenActorsRecursive(const AActor* InActor) const
{
	TSet<TWeakObjectPtr<AActor>> ChildrenWeak;

	if (InActor)
	{
		TArray<AActor*> ReferenceAttachedActors;
		InActor->GetAttachedActors(ReferenceAttachedActors, false, true);
		Algo::Transform(ReferenceAttachedActors, ChildrenWeak, [](AActor* InAttachedActor)->TWeakObjectPtr<AActor>
		{
			return InAttachedActor;
		});
	}

	return ChildrenWeak;
}

TArray<TWeakObjectPtr<AActor>> FActorModifierSceneTreeUpdateExtension::GetDirectChildrenActor(AActor* InActor) const
{
	TArray<TWeakObjectPtr<AActor>> DirectChildrenWeak;

	if (!IsValid(InActor))
	{
		return DirectChildrenWeak;
	}

	TArray<AActor*> DirectChildren;
	bool bSuccess = false;
	
	if (SceneTreeResolver.IsValid())
	{
		bSuccess = SceneTreeResolver->GetDirectChildrenActor(InActor, DirectChildren);
	}
	
	if (!bSuccess)
	{
		// Default
		InActor->GetAttachedActors(DirectChildren, /** Reset */true, /** Recursive */false);
	}
	
	Algo::Transform(DirectChildren, DirectChildrenWeak, [](AActor* InActor)->TWeakObjectPtr<AActor>
	{
		return InActor;
	});

	return DirectChildrenWeak;
}

TArray<TWeakObjectPtr<AActor>> FActorModifierSceneTreeUpdateExtension::GetParentActors(const AActor* InActor) const
{
	TArray<TWeakObjectPtr<AActor>> ActorParentsWeak;

	while (InActor && InActor->GetAttachParentActor())
	{
		AActor* ParentActor = InActor->GetAttachParentActor();
		ActorParentsWeak.Add(ParentActor);
		InActor = ParentActor;
	}

	return ActorParentsWeak;
}

TArray<TWeakObjectPtr<AActor>> FActorModifierSceneTreeUpdateExtension::GetReferenceActors(const FActorModifierSceneTreeActor* InReferenceActor) const
{
	TArray<TWeakObjectPtr<AActor>> ReferenceActors;
	
	if (!InReferenceActor)
	{
		return ReferenceActors;
	}

	AActor* LocalActor = InReferenceActor->GetLocalActor();
	if (!IsValid(LocalActor))
	{
		return ReferenceActors;
	}

	if (InReferenceActor->ReferenceContainer == EActorModifierReferenceContainer::Other)
	{
		if (AActor* ReferenceActor = InReferenceActor->ReferenceActorWeak.Get())
		{
			ReferenceActors.Add(ReferenceActor);
		}

		return ReferenceActors;
	}

	// Note: Using Typed Outer World, instead of GetWorld(), since the typed outer World could be a streamed in world
	// and GetWorld() only returns the main world.
	UWorld* const World = LocalActor->GetTypedOuter<UWorld>();
	if (!IsValid(World))
	{
		return ReferenceActors;
	}

	TArray<TWeakObjectPtr<AActor>> SiblingActors;
	
	// Are we on the root level
	if (AActor* const ParentActor = LocalActor->GetAttachParentActor())
	{
		SiblingActors = GetDirectChildrenActor(ParentActor);
	}
	else
	{
		SiblingActors = GetRootActors(LocalActor->GetLevel());
	}

	if (SiblingActors.IsEmpty())
	{
		return ReferenceActors;
	}

	int32 FromActorIndex = INDEX_NONE;
	int32 ToActorIndex = INDEX_NONE;
	if (!SiblingActors.Find(LocalActor, FromActorIndex))
	{
		return ReferenceActors;
	}

	switch (InReferenceActor->ReferenceContainer)
	{
	case EActorModifierReferenceContainer::Previous:
		{
			ToActorIndex = 0;
		}
		break;
	case EActorModifierReferenceContainer::Next:
		{
			ToActorIndex = SiblingActors.Num() - 1;
		}
		break;
	case EActorModifierReferenceContainer::First:
		{
			FromActorIndex = 0;
			ToActorIndex = 0;
		}
		break;
	case EActorModifierReferenceContainer::Last:
		{
			FromActorIndex = SiblingActors.Num() - 1;
			ToActorIndex = SiblingActors.Num() - 1;
		}
		break;
	default: ;
	}

	if (!SiblingActors.IsValidIndex(FromActorIndex)
		|| !SiblingActors.IsValidIndex(ToActorIndex))
	{
		return ReferenceActors;
	}

	auto SearchReferenceActor = [&SiblingActors, &LocalActor, &ReferenceActors, &InReferenceActor](int32 Index)->bool
	{
		AActor* SiblingActor = SiblingActors[Index].Get();

		if (SiblingActor == LocalActor)
		{
			return true;
		}
		
		ReferenceActors.Add(SiblingActor);
		
		if (!InReferenceActor->bSkipHiddenActors
			|| UE::ActorModifier::ActorUtils::IsActorVisible(SiblingActor))
		{
			return false;
		}

		return true;
	};

	if (FromActorIndex < ToActorIndex)
	{
		for (int32 Index = FromActorIndex; Index <= ToActorIndex; Index++)
		{
			if (!SearchReferenceActor(Index))
			{
				break;
			}
		}
	}
	else if (FromActorIndex > ToActorIndex)
	{
		for (int32 Index = FromActorIndex; Index >= ToActorIndex; Index--)
		{
			if (!SearchReferenceActor(Index))
			{
				break;
			}
		}
	}
	else
	{
		SearchReferenceActor(FromActorIndex);
	}
	
	return ReferenceActors;
}

TArray<TWeakObjectPtr<AActor>> FActorModifierSceneTreeUpdateExtension::GetRootActors(ULevel* InLevel) const
{
	TArray<TWeakObjectPtr<AActor>> RootActorsWeak;
	
	if (!IsValid(InLevel))
	{
		return RootActorsWeak;
	}

	bool bSuccess = false;
	TArray<AActor*> RootActors;

	if (SceneTreeResolver.IsValid())
	{
		bSuccess = SceneTreeResolver->GetRootActors(InLevel, RootActors);
	}

	if (!bSuccess)
	{
		// Default
		RootActors.Empty(InLevel->Actors.Num());

		for (AActor* Actor : InLevel->Actors)
		{
			if (IsValid(Actor) && !Actor->GetAttachParentActor())
			{
				RootActors.Add(Actor);
			}
		}
	}

	Algo::Transform(RootActors, RootActorsWeak, [](AActor* InActor)
	{
		return InActor;
	});
	
	return RootActorsWeak;
}

void FActorModifierSceneTreeUpdateExtension::CheckTrackedActorUpdate(int32 InIdx) const
{
	if (!IsExtensionEnabled())
	{
		return;
	}

	FActorModifierSceneTreeActor* TrackedActor = GetTrackedActor(InIdx);
	if (!TrackedActor)
	{
		return;
	}

	AActor* LocalActor = TrackedActor->LocalActorWeak.IsValid()
		? TrackedActor->LocalActorWeak.Get()
		: GetModifierActor();

	if (!LocalActor)
	{
		return;
	}

	// Reapply in case we overwrite the whole struct outside
	TrackedActor->LocalActorWeak = LocalActor;

	// Gather previous reference actor before clearing array
	AActor* PreviousReferenceActor = !TrackedActor->ReferenceActorsWeak.IsEmpty()
		? TrackedActor->ReferenceActorsWeak.Last().Get()
		: nullptr;

	// Track siblings actors in case their visibility changes
	TrackedActor->ReferenceActorsWeak = GetReferenceActors(TrackedActor);
	AActor* NewReferenceActor = !TrackedActor->ReferenceActorsWeak.IsEmpty() ? TrackedActor->ReferenceActorsWeak.Last().Get() : nullptr;
	TrackedActor->ReferenceActorWeak = NewReferenceActor;

	// Gather children of reference actor
	const TSet<TWeakObjectPtr<AActor>> PreviousReferenceActorChildrenWeak = TrackedActor->ReferenceActorChildrenWeak;
	TrackedActor->ReferenceActorChildrenWeak = GetChildrenActorsRecursive(NewReferenceActor);

	// Gather direct children ordered of reference actor
	const TArray<TWeakObjectPtr<AActor>> PreviousDirectChildrenWeak = TrackedActor->ReferenceActorDirectChildrenWeak;
	TrackedActor->ReferenceActorDirectChildrenWeak = GetDirectChildrenActor(NewReferenceActor);

	// Gather parents of reference actor
	const TArray<TWeakObjectPtr<AActor>> PreviousParentActorsWeak = TrackedActor->ReferenceActorParentsWeak;
	TrackedActor->ReferenceActorParentsWeak = GetParentActors(NewReferenceActor);

	const bool bReferenceActorChanged = NewReferenceActor != PreviousReferenceActor;
	const bool bReferenceActorChildrenChanged = !(TrackedActor->ReferenceActorChildrenWeak.Num() == PreviousReferenceActorChildrenWeak.Num()
			&& TrackedActor->ReferenceActorChildrenWeak.Difference(PreviousReferenceActorChildrenWeak).IsEmpty());
	const bool bReferenceActorDirectChildrenChanged = !IsSameActorArray(PreviousDirectChildrenWeak, TrackedActor->ReferenceActorDirectChildrenWeak);
	const bool bReferenceActorParentChanged = !IsSameActorArray(PreviousParentActorsWeak, TrackedActor->ReferenceActorParentsWeak);

	if (IActorModifierSceneTreeUpdateHandler* HandlerInterface = ExtensionHandlerWeak.Get())
	{
		// Fire event when reference actor changed
		if (bReferenceActorChanged)
		{
			HandlerInterface->OnSceneTreeTrackedActorChanged(InIdx, PreviousReferenceActor, NewReferenceActor);
		}

		// Fire event when children actors changed
		if (bReferenceActorChildrenChanged)
		{
			HandlerInterface->OnSceneTreeTrackedActorChildrenChanged(InIdx, PreviousReferenceActorChildrenWeak, TrackedActor->ReferenceActorChildrenWeak);
		}

		// Fire event when direct children actors changed (even order)
		if (bReferenceActorDirectChildrenChanged)
		{
			HandlerInterface->OnSceneTreeTrackedActorDirectChildrenChanged(InIdx, PreviousDirectChildrenWeak, TrackedActor->ReferenceActorDirectChildrenWeak);
		}

		if (bReferenceActorParentChanged)
		{
			HandlerInterface->OnSceneTreeTrackedActorParentChanged(InIdx, PreviousParentActorsWeak, TrackedActor->ReferenceActorParentsWeak);
		}
	}
}
