// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/ActorModifierTransformUpdateExtension.h"

#include "Modifiers/ActorModifierCoreBase.h"

FActorModifierTransformUpdateExtension::FActorModifierTransformUpdateExtension(IActorModifierTransformUpdateHandler* InExtensionInterface)
	: ExtensionHandlerWeak(InExtensionInterface)
{
	check(InExtensionInterface);
}

void FActorModifierTransformUpdateExtension::TrackActor(AActor* InActor, bool bInReset)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return;
	}

	if (bInReset)
	{
		const TSet<TWeakObjectPtr<AActor>> NewSet {InActor};
		UntrackActors(TrackedActors.Difference(NewSet));
	}
	
	if (TrackedActors.Contains(InActor))
	{
		return;
	}

	if (IsExtensionEnabled())
	{
		BindDelegate(InActor);
	}

	TrackedActors.Add(InActor);
}

void FActorModifierTransformUpdateExtension::UntrackActor(AActor* InActor)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return;
	}
	
	if (!TrackedActors.Contains(InActor))
	{
		return;
	}

	UnbindDelegate(InActor);

	TrackedActors.Remove(InActor);
}

void FActorModifierTransformUpdateExtension::TrackActors(const TSet<TWeakObjectPtr<AActor>>& InActors, bool bInReset)
{
	if (bInReset)
	{
		UntrackActors(TrackedActors.Difference(InActors));	
	}
	
	for (const TWeakObjectPtr<AActor>& InActor : InActors)
	{
		TrackActor(InActor.Get(), false);
	}
}

void FActorModifierTransformUpdateExtension::UntrackActors(const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	for (const TWeakObjectPtr<AActor>& InActor : InActors)
	{
		UntrackActor(InActor.Get());
	}
}

void FActorModifierTransformUpdateExtension::OnExtensionEnabled(EActorModifierCoreEnableReason InReason)
{
	for (TWeakObjectPtr<AActor>& TrackedActor : TrackedActors)
	{
		BindDelegate(TrackedActor.Get());
	}
}

void FActorModifierTransformUpdateExtension::OnExtensionDisabled(EActorModifierCoreDisableReason InReason)
{
	for (TWeakObjectPtr<AActor>& TrackedActor : TrackedActors)
	{
		UnbindDelegate(TrackedActor.Get());
	}
}

void FActorModifierTransformUpdateExtension::OnTransformUpdated(USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InType)
{
	if (!InComponent)
	{
		return;
	}

	AActor* ActorTransformed = InComponent->GetOwner();
	if (!ActorTransformed)
	{
		return;
	}

	const UActorModifierCoreBase* Modifier = GetModifier();
	if (!Modifier || !Modifier->IsModifierEnabled())
	{
		return;
	}
	
	if (IActorModifierTransformUpdateHandler* HandlerInterface = ExtensionHandlerWeak.Get())
	{
		HandlerInterface->OnTransformUpdated(ActorTransformed, InFlags == EUpdateTransformFlags::PropagateFromParent);
	}
}

void FActorModifierTransformUpdateExtension::BindDelegate(const AActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	if (USceneComponent* SceneComponent = InActor->GetRootComponent())
	{
		SceneComponent->TransformUpdated.RemoveAll(this);
		SceneComponent->TransformUpdated.AddSP(this, &FActorModifierTransformUpdateExtension::OnTransformUpdated);
	}
}

void FActorModifierTransformUpdateExtension::UnbindDelegate(const AActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	if (USceneComponent* SceneComponent = InActor->GetRootComponent())
	{
		SceneComponent->TransformUpdated.RemoveAll(this);
	}
}
