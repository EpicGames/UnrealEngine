// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierArrangeBaseModifier.h"

#include "GameFramework/Actor.h"
#include "Misc/ITransaction.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Shared/ActorModifierVisibilityShared.h"

void UActorModifierArrangeBaseModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierRenderStateUpdateExtension>(this);
	AddExtension<FActorModifierTransformUpdateExtension>(this);

	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
		ReferenceActor.ReferenceActorWeak = GetModifiedActor();
		ReferenceActor.bSkipHiddenActors = false;
		
		SceneExtension->TrackSceneTree(0, &ReferenceActor);
	}
}

void UActorModifierArrangeBaseModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(false))
	{
		LayoutShared->RestoreActorsState(this);
	}

	if (UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(false))
	{
		VisibilityShared->RestoreActorsState(this);
	}
}

void UActorModifierArrangeBaseModifier::OnModifiedActorTransformed()
{
	Super::OnModifiedActorTransformed();
}

void UActorModifierArrangeBaseModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx,
	const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	const AActor* const ModifyActor = GetModifiedActor();
	if (!IsValid(ModifyActor))
	{
		return;
	}

	MarkModifierDirty();
}

void UActorModifierArrangeBaseModifier::OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx,
	const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorDirectChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	const AActor* const ModifyActor = GetModifiedActor();
	if (!IsValid(ModifyActor))
	{
		return;
	}

	MarkModifierDirty();
}
