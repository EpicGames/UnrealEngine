// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierAttachmentBaseModifier.h"

#include "GameFramework/Actor.h"

void UActorModifierAttachmentBaseModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);
	
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor && InActor->FindComponentByClass<USceneComponent>();		
	});
}

void UActorModifierAttachmentBaseModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierSceneTreeUpdateExtension>(this);
}
