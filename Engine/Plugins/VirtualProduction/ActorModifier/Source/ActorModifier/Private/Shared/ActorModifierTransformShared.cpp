// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/ActorModifierTransformShared.h"

#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierCoreBase.h"

void FActorModifierTransformSharedModifierState::Save(const AActor* InActor, EActorModifierTransformSharedState InSaveState)
{
	if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
	{
		if (InActor)
		{
			if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Location)
				&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Location))
			{
				ActorTransform.SetLocation(InActor->GetActorLocation());
				EnumAddFlags(SaveState, EActorModifierTransformSharedState::Location);
			}

			if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Rotation)
				&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Rotation))
			{
				ActorTransform.SetRotation(InActor->GetActorRotation().Quaternion());
				EnumAddFlags(SaveState, EActorModifierTransformSharedState::Rotation);
			}

			if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Scale)
				&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Scale))
			{
				ActorTransform.SetScale3D(InActor->GetActorScale3D());
				EnumAddFlags(SaveState, EActorModifierTransformSharedState::Scale);
			}
		}
	}
}

void FActorModifierTransformSharedModifierState::Restore(AActor* InActor, EActorModifierTransformSharedState InRestoreState)
{
	if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
	{
		if (InActor)
		{
			FTransform RestoreTransform = ActorTransform;
			const FTransform& CurrentActorTransform = InActor->GetActorTransform();

			if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Location)
				|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Location))
			{
				RestoreTransform.SetLocation(CurrentActorTransform.GetLocation());
			}

			if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Rotation)
				|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Rotation))
			{
				RestoreTransform.SetRotation(CurrentActorTransform.GetRotation());
			}

			if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Scale)
				|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Scale))
			{
				RestoreTransform.SetScale3D(CurrentActorTransform.GetScale3D());
			}

			if (!CurrentActorTransform.Equals(RestoreTransform))
			{
				InActor->SetActorTransform(RestoreTransform);
			}

			EnumRemoveFlags(SaveState, InRestoreState);
		}
	}
}

void FActorModifierTransformSharedActorState::Save(EActorModifierTransformSharedState InSaveState)
{
	if (const AActor* Actor = ActorWeak.Get())
	{
		if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Location)
			&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Location))
		{
			ActorTransform.SetLocation(Actor->GetActorLocation());
			EnumAddFlags(SaveState, EActorModifierTransformSharedState::Location);
		}

		if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Rotation)
			&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Rotation))
		{
			ActorTransform.SetRotation(Actor->GetActorRotation().Quaternion());
			EnumAddFlags(SaveState, EActorModifierTransformSharedState::Rotation);
		}

		if (EnumHasAnyFlags(InSaveState, EActorModifierTransformSharedState::Scale)
			&& !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Scale))
		{
			ActorTransform.SetScale3D(Actor->GetActorScale3D());
			EnumAddFlags(SaveState, EActorModifierTransformSharedState::Scale);
		}
	}
}

void FActorModifierTransformSharedActorState::Restore(EActorModifierTransformSharedState InRestoreState)
{
	if (AActor* Actor = ActorWeak.Get())
	{
		FTransform RestoreTransform = ActorTransform;
		const FTransform& CurrentActorTransform = Actor->GetActorTransform();

		if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Location)
			|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Location))
		{
			RestoreTransform.SetLocation(CurrentActorTransform.GetLocation());
		}

		if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Rotation)
			|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Rotation))
		{
			RestoreTransform.SetRotation(CurrentActorTransform.GetRotation());
		}

		if (!EnumHasAnyFlags(InRestoreState, EActorModifierTransformSharedState::Scale)
			|| !EnumHasAnyFlags(SaveState, EActorModifierTransformSharedState::Scale))
		{
			RestoreTransform.SetScale3D(CurrentActorTransform.GetScale3D());
		}

		if (!CurrentActorTransform.Equals(RestoreTransform))
		{
			Actor->SetActorTransform(RestoreTransform);
		}

		EnumRemoveFlags(SaveState, InRestoreState);
	}
}

void UActorModifierTransformShared::SaveActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor, EActorModifierTransformSharedState InSaveState)
{
	if (!IsValid(InActor))
	{
		return;
	}

	FActorModifierTransformSharedActorState& ActorState = ActorStates.FindOrAdd(FActorModifierTransformSharedActorState(InActor));
	ActorState.Save(InSaveState);

	FActorModifierTransformSharedModifierState& ModifierState = ActorState.ModifierStates.FindOrAdd(FActorModifierTransformSharedModifierState(InModifierContext));
	ModifierState.Save(InActor, InSaveState);
}

void UActorModifierTransformShared::RestoreActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor, EActorModifierTransformSharedState InRestoreState)
{
	if (!IsValid(InActor))
	{
		return;
	}

	FActorModifierTransformSharedActorState* ActorState = ActorStates.Find(FActorModifierTransformSharedActorState(InActor));
	if (!ActorState)
	{
		return;
	}

	FActorModifierTransformSharedModifierState* ActorModifierState = ActorState->ModifierStates.Find(FActorModifierTransformSharedModifierState(InModifierContext));
	if (!ActorModifierState)
	{
		return;
	}

	// restore modifier state and remove it
	ActorModifierState->Restore(InActor, InRestoreState);

	if (ActorModifierState->SaveState == EActorModifierTransformSharedState::None)
	{
		ActorState->ModifierStates.Remove(*ActorModifierState);
	}

	// Restore original actor state and remove it
	if (ActorState->ModifierStates.IsEmpty())
	{
		ActorState->Restore(EActorModifierTransformSharedState::All);
		ActorStates.Remove(*ActorState);
	}
}

FActorModifierTransformSharedActorState* UActorModifierTransformShared::FindActorState(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return nullptr;
	}

	return ActorStates.Find(FActorModifierTransformSharedActorState(InActor));
}

TSet<FActorModifierTransformSharedActorState*> UActorModifierTransformShared::FindActorsState(UActorModifierCoreBase* InModifierContext)
{
	TSet<FActorModifierTransformSharedActorState*> ModifierActorStates;

	for (FActorModifierTransformSharedActorState& ActorState : ActorStates)
	{
		if (ActorState.ModifierStates.Contains(FActorModifierTransformSharedModifierState(InModifierContext)))
		{
			ModifierActorStates.Add(&ActorState);
		}
	}

	return ModifierActorStates;
}

void UActorModifierTransformShared::RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<AActor*>* InActors, EActorModifierTransformSharedState InRestoreState)
{
	const FActorModifierTransformSharedModifierState SearchModifierState(InModifierContext);
	TSet<AActor*> LinkedModifierActors;
	TSet<UActorModifierCoreBase*> LinkedActorModifiers;

	for (const FActorModifierTransformSharedActorState& ActorState : ActorStates)
	{
		AActor* Actor = ActorState.ActorWeak.Get();
		if (!Actor)
		{
			continue;
		}

		if (!ActorState.ModifierStates.Contains(SearchModifierState))
		{
			continue;
		}

		if (InActors && !InActors->Contains(Actor))
		{
			continue;
		}

		// Collect actors affected by modifier
		LinkedModifierActors.Add(Actor);

		// Collect linked actor modifiers
		for (const FActorModifierTransformSharedModifierState& ModifierState : ActorState.ModifierStates)
		{
			if (UActorModifierCoreBase* Modifier = ModifierState.ModifierWeak.Get())
			{
				LinkedActorModifiers.Add(Modifier);
			}
		}
	}

	// Locking state to prevent from updating when restoring state
	// When destroyed : Unlocking state of modifier
	FActorModifierCoreScopedLock ModifiersLock(LinkedActorModifiers);

	// Restore actor state
	for (AActor* Actor : LinkedModifierActors)
	{
		RestoreActorState(InModifierContext, Actor, InRestoreState);
	}
}

void UActorModifierTransformShared::RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<TWeakObjectPtr<AActor>>& InActors, EActorModifierTransformSharedState InRestoreState)
{
	TSet<AActor*> Actors;
	Algo::Transform(InActors, Actors, [](const TWeakObjectPtr<AActor>& InActor)->AActor*{ return InActor.Get(); });

	RestoreActorsState(InModifierContext, &Actors, InRestoreState);
}

bool UActorModifierTransformShared::IsActorStateSaved(UActorModifierCoreBase* InModifierContext, AActor* InActor)
{
	if (const FActorModifierTransformSharedActorState* ActorState = FindActorState(InActor))
	{
		return ActorState->ModifierStates.Contains(FActorModifierTransformSharedModifierState(InModifierContext));
	}

	return false;
}

bool UActorModifierTransformShared::IsActorsStateSaved(UActorModifierCoreBase* InModifierContext)
{
	const FActorModifierTransformSharedModifierState ModifierState(InModifierContext);

	for (const FActorModifierTransformSharedActorState& ActorState : ActorStates)
	{
		if (ActorState.ModifierStates.Contains(ModifierState))
		{
			return true;
		}
	}

	return false;
}

void UActorModifierTransformShared::PostLoad()
{
	Super::PostLoad();

	// Remove invalid items when loading
	for (FActorModifierTransformSharedActorState& ActorState : ActorStates)
	{
		ActorState.ModifierStates.Remove(FActorModifierTransformSharedModifierState(nullptr));
	}
}
