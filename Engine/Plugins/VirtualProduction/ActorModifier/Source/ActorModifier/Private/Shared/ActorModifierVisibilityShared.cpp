// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/ActorModifierVisibilityShared.h"

#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"

void FActorModifierVisibilitySharedModifierState::Save(const AActor* InActor)
{
	if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
	{
		if (InActor)
		{
			UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Visibility);
#if WITH_EDITOR
			bActorHiddenInEditor = InActor->IsTemporarilyHiddenInEditor();
#endif
			bActorHiddenInGame = InActor->IsHidden();
		}
	}
}

void FActorModifierVisibilitySharedModifierState::Restore(AActor* InActor) const
{
	if (const UActorModifierCoreBase* Modifier = ModifierWeak.Get())
	{
		if (InActor)
		{
			UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Visibility);
#if WITH_EDITOR
			InActor->SetIsTemporarilyHiddenInEditor(bActorHiddenInEditor);
#endif
			InActor->SetActorHiddenInGame(bActorHiddenInGame);
		}
	}
}

void FActorModifierVisibilitySharedActorState::Save()
{
	if (const AActor* Actor = ActorWeak.Get())
	{
#if WITH_EDITOR
		bActorHiddenInEditor = Actor->IsTemporarilyHiddenInEditor();
#endif
		bActorHiddenInGame = Actor->IsHidden();
	}
}

void FActorModifierVisibilitySharedActorState::Restore() const
{
	if (AActor* Actor = ActorWeak.Get())
	{
		UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Visibility);
#if WITH_EDITOR
		Actor->SetIsTemporarilyHiddenInEditor(bActorHiddenInEditor);
#endif
		Actor->SetActorHiddenInGame(bActorHiddenInGame);
	}
}

void UActorModifierVisibilityShared::SaveActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}

	FActorModifierVisibilitySharedActorState& ActorState = ActorStates.FindOrAdd(FActorModifierVisibilitySharedActorState(InActor));

	if (ActorState.ModifierStates.IsEmpty())
	{
		ActorState.Save();
	}
	
	FActorModifierVisibilitySharedModifierState ModifierState(InModifierContext);
	if (!ActorState.ModifierStates.Contains(ModifierState))
	{
		ModifierState.Save(InActor);
		ActorState.ModifierStates.Add(InModifierContext);
	}
}

void UActorModifierVisibilityShared::RestoreActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return;
	}

	if (FActorModifierVisibilitySharedActorState* ActorState = ActorStates.Find(FActorModifierVisibilitySharedActorState(InActor)))
	{
		if (const FActorModifierVisibilitySharedModifierState* ActorModifierState = ActorState->ModifierStates.Find(FActorModifierVisibilitySharedModifierState(InModifierContext)))
		{
			// restore modifier state and remove it
			ActorModifierState->Restore(InActor);
			ActorState->ModifierStates.Remove(*ActorModifierState);

			// Restore original actor state and remove it
			if (ActorState->ModifierStates.IsEmpty())
			{
				ActorState->Restore();
				ActorStates.Remove(*ActorState);
			}
		}
	}
}

FActorModifierVisibilitySharedActorState* UActorModifierVisibilityShared::FindActorState(AActor* InActor)
{
	if (!IsValid(InActor))
	{
		return nullptr;
	}

	return ActorStates.Find(FActorModifierVisibilitySharedActorState(InActor));
}

void UActorModifierVisibilityShared::SetActorVisibility(UActorModifierCoreBase* InModifierContext, AActor* InActor, bool bInHidden, bool bInRecurse, EActorModifierVisibilityActor InActorVisibility)
{
	if (!IsValid(InActor))
	{
		return;
	}

	TArray<AActor*> Actors { InActor };

	if (bInRecurse)
	{
		constexpr bool bResetArray = false;
		constexpr bool bRecursivelyIncludeAttachedActors = true;
		InActor->GetAttachedActors(Actors, bResetArray, bRecursivelyIncludeAttachedActors);
	}

	SetActorsVisibility(InModifierContext, Actors, bInHidden, InActorVisibility);
}

void UActorModifierVisibilityShared::SetActorsVisibility(UActorModifierCoreBase* InModifierContext, TArray<AActor*> InActors, bool bInHidden, EActorModifierVisibilityActor InActorVisibility)
{
	for (AActor* Actor : InActors)
	{
		if (!Actor)
		{
			continue;
		}
		
		SaveActorState(InModifierContext, Actor);

		UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Visibility);

#if WITH_EDITOR
		if (EnumHasAnyFlags(InActorVisibility, EActorModifierVisibilityActor::Editor))
		{
			if (Actor->IsTemporarilyHiddenInEditor() != bInHidden)
			{
				Actor->SetIsTemporarilyHiddenInEditor(bInHidden);
			}
		}
#endif
		
		if (EnumHasAnyFlags(InActorVisibility, EActorModifierVisibilityActor::Game))
		{
			if (Actor->IsHidden() != bInHidden)
			{
				Actor->SetActorHiddenInGame(bInHidden);
			}
		}
	}
}

void UActorModifierVisibilityShared::RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<AActor*>* InActors)
{
	const FActorModifierVisibilitySharedModifierState SearchModifierState(InModifierContext);
	TSet<AActor*> LinkedModifierActors;
	TSet<UActorModifierCoreBase*> LinkedActorModifiers;
	
	for (const FActorModifierVisibilitySharedActorState& ActorState : ActorStates)
	{
		AActor* Actor = ActorState.ActorWeak.Get();
		if (!Actor)
		{
			continue;
		}

		if (!ActorState.ModifierStates.Contains(SearchModifierState))
		{
			continue;
		}

		if (InActors && !InActors->Contains(Actor))
		{
			continue;
		}
		
		// Collect actors affected by modifier
		LinkedModifierActors.Add(Actor);

		// Collect linked actor modifiers
		for (const FActorModifierVisibilitySharedModifierState& ModifierState : ActorState.ModifierStates)
		{
			if (UActorModifierCoreBase* Modifier = ModifierState.ModifierWeak.Get())
			{
				LinkedActorModifiers.Add(Modifier);
			}
		}
	}

	// Locking state to prevent from updating when restoring state
	// When destroyed : Unlocking state of modifier
	FActorModifierCoreScopedLock ModifiersLock(LinkedActorModifiers);

	// Restore actor state
	for (AActor* Actor : LinkedModifierActors)
	{
		RestoreActorState(InModifierContext, Actor);
	}
}

void UActorModifierVisibilityShared::RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	TSet<AActor*> Actors;
	Algo::Transform(InActors, Actors, [](const TWeakObjectPtr<AActor>& InActor)->AActor*{ return InActor.Get(); });

	RestoreActorsState(InModifierContext, &Actors);
}

bool UActorModifierVisibilityShared::IsActorStateSaved(UActorModifierCoreBase* InModifierContext, AActor* InActor)
{
	if (const FActorModifierVisibilitySharedActorState* ActorState = FindActorState(InActor))
	{
		return ActorState->ModifierStates.Contains(FActorModifierVisibilitySharedModifierState(InModifierContext));
	}
	
	return false;
}

bool UActorModifierVisibilityShared::IsActorsStateSaved(UActorModifierCoreBase* InModifierContext)
{
	const FActorModifierVisibilitySharedModifierState ModifierState(InModifierContext);
	
	for (const FActorModifierVisibilitySharedActorState& ActorState : ActorStates)
	{
		if (ActorState.ModifierStates.Contains(ModifierState))
		{
			return true;
		}
	}
	
	return false;
}

void UActorModifierVisibilityShared::PostLoad()
{
	Super::PostLoad();

	// Remove invalid items when loading
	for (FActorModifierVisibilitySharedActorState& ActorState : ActorStates)
	{
		ActorState.ModifierStates.Remove(FActorModifierVisibilitySharedModifierState(nullptr));
	}
}
