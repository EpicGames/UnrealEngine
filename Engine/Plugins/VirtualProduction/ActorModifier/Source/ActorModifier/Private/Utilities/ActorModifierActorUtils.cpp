// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/ActorModifierActorUtils.h"

#include "Components/PrimitiveComponent.h"
#include "Containers/Set.h"
#include "GameFramework/Actor.h"
#include "UObject/WeakObjectPtr.h"

bool UE::ActorModifier::ActorUtils::IsActorVisible(const AActor* InActor)
{
	if (!InActor)
	{
		return false;
	}

	const bool bIsActorVisible = !InActor->IsHidden();

#if WITH_EDITOR
	const bool bIsActorVisibleInEditor = !InActor->IsTemporarilyHiddenInEditor();
#else
	constexpr bool bIsActorVisibleInEditor = true;
#endif

	// If we don't have a root component, consider this true to skip it in final condition
	const bool bIsRootComponentVisible = !InActor->GetRootComponent() || InActor->GetRootComponent()->IsVisible();

	return bIsActorVisible && bIsActorVisibleInEditor && bIsRootComponentVisible;
}

FBox UE::ActorModifier::ActorUtils::GetActorsBounds(const TSet<TWeakObjectPtr<AActor>>& InActors, const FTransform& InReferenceTransform, bool bInSkipHidden, bool bInTransformBox)
{
	FBox ActorsBounds = FBox(EForceInit::ForceInit);
	ActorsBounds.IsValid = 0;

	FVector OrientedVerts[8];

	FTransform AccumulatedTransform = InReferenceTransform;

	for (const TWeakObjectPtr<AActor>& ActorWeak : InActors)
	{
		const AActor* Actor = ActorWeak.Get();

		if (!Actor)
		{
			continue;
		}

		if (bInSkipHidden && !UE::ActorModifier::ActorUtils::IsActorVisible(Actor))
		{
			continue;
		}

		const FBox ActorBounds = UE::ActorModifier::ActorUtils::GetActorBounds(Actor);

		if (ActorBounds.IsValid > 0)
		{
			FTransform ActorTransform = Actor->GetTransform();
			ActorTransform.SetScale3D(FVector::OneVector);

			const FOrientedBox OrientedBox = UE::ActorModifier::ActorUtils::GetOrientedBox(ActorBounds, ActorTransform);

			OrientedBox.CalcVertices(OrientedVerts);

			for (const FVector& OrientedVert : OrientedVerts)
			{
				ActorsBounds += AccumulatedTransform.InverseTransformPositionNoScale(OrientedVert);
			}

			ActorsBounds.IsValid = 1;
		}
	}

	AccumulatedTransform.SetScale3D(FVector::OneVector);

	return bInTransformBox ? ActorsBounds.TransformBy(AccumulatedTransform) : ActorsBounds;
}

FBox UE::ActorModifier::ActorUtils::GetActorsBounds(const TSet<TWeakObjectPtr<AActor>>& InActors, const FTransform& InReferenceTransform, bool bInSkipHidden)
{
	constexpr bool bTransformBox = true;
	return GetActorsBounds(InActors, InReferenceTransform, bInSkipHidden, bTransformBox);
}

FBox UE::ActorModifier::ActorUtils::GetActorsBounds(AActor* InActor, bool bInIncludeChildren, bool bInSkipHidden)
{
	if (!InActor)
	{
		return FBox();
	}

	TSet<TWeakObjectPtr<AActor>> AttachedModifyActors {InActor};

	if (bInIncludeChildren)
	{
		TArray<AActor*> AttachedActors;
		InActor->GetAttachedActors(AttachedActors, false, true);
		Algo::Transform(AttachedActors, AttachedModifyActors, [](AActor* InAttachedActor)
		{
			return InAttachedActor;
		});
	}

	return UE::ActorModifier::ActorUtils::GetActorsBounds(AttachedModifyActors, InActor->GetActorTransform(), bInSkipHidden);
}

FBox UE::ActorModifier::ActorUtils::GetActorBounds(const AActor* InActor)
{
	FBox Box(ForceInit);
	Box.IsValid = 0;

	if (!InActor || !InActor->GetRootComponent())
	{
		return Box;
	}

	FTransform ActorToWorld = InActor->GetTransform();
	ActorToWorld.SetScale3D(FVector::OneVector);
	const FTransform WorldToActor = ActorToWorld.Inverse();

	InActor->ForEachComponent<UPrimitiveComponent>(true, [&WorldToActor, &Box](const UPrimitiveComponent* InPrimitiveComponent)
	{
		if (!IsValid(InPrimitiveComponent))
		{
			return;
		}

#if WITH_EDITOR
		// Ignore Visualization Components, but don't consider them as failed components.
		if (InPrimitiveComponent->IsVisualizationComponent())
		{
			return;
		}
#endif

		const FTransform ComponentToActor = InPrimitiveComponent->GetComponentTransform() * WorldToActor;
		const FBox ComponentBox = InPrimitiveComponent->CalcBounds(ComponentToActor).GetBox();

		Box += ComponentBox;
		Box.IsValid = 1;
	});

	return Box;
}

FVector UE::ActorModifier::ActorUtils::GetVectorAxis(int32 InAxis)
{
	FVector FollowedAxisVector = FVector::ZeroVector;

	if (EnumHasAnyFlags(static_cast<EActorModifierAxis>(InAxis), EActorModifierAxis::X))
	{
		FollowedAxisVector.X = 1;
	}

	if (EnumHasAnyFlags(static_cast<EActorModifierAxis>(InAxis), EActorModifierAxis::Y))
	{
		FollowedAxisVector.Y = 1;
	}

	if (EnumHasAnyFlags(static_cast<EActorModifierAxis>(InAxis), EActorModifierAxis::Z))
	{
		FollowedAxisVector.Z = 1;
	}

	return FollowedAxisVector;
}

bool UE::ActorModifier::ActorUtils::IsAxisVectorEquals(const FVector& InVectorA, const FVector& InVectorB, int32 InCompareAxis)
{
	// Compare InVectorA and InVectorB but only component of InCompareAxis
	const FVector FollowedAxisVector = UE::ActorModifier::ActorUtils::GetVectorAxis(InCompareAxis);
	if ((InVectorA * FollowedAxisVector).Equals(InVectorB * FollowedAxisVector, 0.1))
	{
		return true;
	}

	return false;
}

FOrientedBox UE::ActorModifier::ActorUtils::GetOrientedBox(const FBox& InLocalBox, const FTransform& InWorldTransform)
{
	FOrientedBox OutOrientedBox;
	OutOrientedBox.Center = InWorldTransform.TransformPosition(InLocalBox.GetCenter());

	OutOrientedBox.AxisX = InWorldTransform.TransformVector(FVector::UnitX());
	OutOrientedBox.AxisY = InWorldTransform.TransformVector(FVector::UnitY());
	OutOrientedBox.AxisZ = InWorldTransform.TransformVector(FVector::UnitZ());

	OutOrientedBox.ExtentX = (InLocalBox.Max.X - InLocalBox.Min.X) / 2.f;
	OutOrientedBox.ExtentY = (InLocalBox.Max.Y - InLocalBox.Min.Y) / 2.f;
	OutOrientedBox.ExtentZ = (InLocalBox.Max.Z - InLocalBox.Min.Z) / 2.f;
	return OutOrientedBox;
}

FRotator UE::ActorModifier::ActorUtils::FindLookAtRotation(const FVector& InEyePosition, const FVector& InTargetPosition, EActorModifierAxis InAxis, bool bInFlipAxis)
{
	const FVector Direction = bInFlipAxis ? (InEyePosition - InTargetPosition).GetSafeNormal() : (InTargetPosition - InEyePosition).GetSafeNormal();

	if (Direction.IsNearlyZero())
	{
		return FRotator::ZeroRotator;
	}

	const FRotator BaseRotation = Direction.Rotation();

	FQuat AxisQuat;
	switch (InAxis)
	{
	case EActorModifierAxis::X:
		AxisQuat = FQuat::Identity;
		break;

	case EActorModifierAxis::Y:
		AxisQuat = FQuat(FVector::ZAxisVector, FMath::DegreesToRadians(-90.0f));
		break;

	case EActorModifierAxis::Z:
		AxisQuat = FQuat(FVector::YAxisVector, FMath::DegreesToRadians(90.0f));
		break;

	default:
		return FRotator::ZeroRotator;
	}

	const FQuat FinalQuat = BaseRotation.Quaternion() * AxisQuat;

	return FinalQuat.Rotator();
}