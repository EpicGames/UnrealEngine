// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ActorModifierTypes.generated.h"

UENUM(BlueprintType, meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor="true"))
enum class EActorModifierAxis : uint8
{
	None = 0 UMETA(Hidden),
	X = 1 << 0,
	Y = 1 << 1,
	Z = 1 << 2,
};
ENUM_CLASS_FLAGS(EActorModifierAxis);

UENUM(BlueprintType)
enum class EActorModifierVerticalAlignment : uint8
{
	Top,
	Center,
	Bottom,
};

UENUM(BlueprintType)
enum class EActorModifierHorizontalAlignment : uint8
{
	Left,
	Center,
	Right,
};

UENUM(BlueprintType)
enum class EActorModifierDepthAlignment : uint8
{
	Front,
	Center,
	Back
};

UENUM(BlueprintType)
enum class EActorModifierAlignment : uint8
{
	Horizontal,
	Vertical,
	Depth
};

/** Specifies a set of anchor alignments, one for each 3D axis. */
USTRUCT(BlueprintType)
struct FActorModifierAnchorAlignment
{
	GENERATED_BODY()

	UPROPERTY()
	bool bUseHorizontal = true;
	
	UPROPERTY()
	bool bUseVertical = true;
	
	UPROPERTY()
	bool bUseDepth = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnchorAlignment", meta=(EditCondition="bUseHorizontal", EditConditionHides))
	EActorModifierHorizontalAlignment Horizontal = EActorModifierHorizontalAlignment::Center;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnchorAlignment", meta=(EditCondition="bUseVertical", EditConditionHides))
	EActorModifierVerticalAlignment Vertical = EActorModifierVerticalAlignment::Center;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="AnchorAlignment", meta=(EditCondition="bUseDepth", EditConditionHides))
	EActorModifierDepthAlignment Depth = EActorModifierDepthAlignment::Center;

	FActorModifierAnchorAlignment() {}
	FActorModifierAnchorAlignment(const EActorModifierHorizontalAlignment InHorizontal, const EActorModifierVerticalAlignment InVertical, const EActorModifierDepthAlignment InDepth)
		: Horizontal(InHorizontal), Vertical(InVertical), Depth(InDepth)
	{}

	bool operator==(const FActorModifierAnchorAlignment& Other) const
	{
		return Horizontal == Other.Horizontal && Vertical == Other.Vertical && Depth == Other.Depth;
	}

	bool IsHorizontalMatch(const FActorModifierAnchorAlignment& Other) const { return Horizontal == Other.Horizontal; }
	bool IsVerticalMatch(const FActorModifierAnchorAlignment& Other) const { return Vertical == Other.Vertical; }
	bool IsDepthMatch(const FActorModifierAnchorAlignment& Other) const { return Depth == Other.Depth; }

	/** Returns a point on the bounds extent determined by the alignment properties of the structure. */
	ACTORMODIFIER_API FVector LocalBoundsOffset(const FBox& InBounds, const bool bInInverted = false) const;
};