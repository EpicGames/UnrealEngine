// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreExtension.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "UObject/WeakInterfacePtr.h"
#include "ActorModifierRenderStateUpdateExtension.generated.h"

class AActor;
class UActorComponent;

UINTERFACE(MinimalAPI, NotBlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UActorModifierRenderStateUpdateHandler : public UInterface
{
	GENERATED_BODY()
};

/** Implement this interface to handle extension event */
class IActorModifierRenderStateUpdateHandler
{
	GENERATED_BODY()

public:
	/** 
	 * Returns the relevant render state dirty event types relevant to this handler 
	 * Defaults to all for legacy
	 */
	virtual UE::ActorModifierCore::ERenderStateDirtyReason GetRenderStateDirtyEventTypes() const
	{
		return UE::ActorModifierCore::ERenderStateDirtyReason::All;
	}

	/** Callback when a render state actor in this world changes */
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) = 0;

	/** Callback when a tracked actor visibility has changed */
	virtual void OnActorVisibilityChanged(AActor* InActor) = 0;
};

/**
 * This extension tracks specific actors for render state updates,
 * when an update happens it will invoke IAvaRenderStateUpdateExtension function
 */
class FActorModifierRenderStateUpdateExtension : public FActorModifierCoreExtension
{

public:
	ACTORMODIFIER_API explicit FActorModifierRenderStateUpdateExtension(TNotNull<IActorModifierRenderStateUpdateHandler*> InExtensionHandler);

	/** Adds an actor to track for visibility */
	ACTORMODIFIER_API void TrackActorVisibility(AActor* InActor);

	/** Removes a tracked actor for visibility */
	ACTORMODIFIER_API void UntrackActorVisibility(AActor* InActor);

	/** Checks if actor is tracked for visibility */
	ACTORMODIFIER_API bool IsActorVisibilityTracked(AActor* InActor) const;

	/** Sets current tracked actors, removes any actors not included */
	ACTORMODIFIER_API void SetTrackedActorsVisibility(const TSet<TWeakObjectPtr<AActor>>& InActors);

	/** Sets current tracked actors with actor and its children, removes any actors not included */
	ACTORMODIFIER_API void SetTrackedActorVisibility(AActor* InActor, bool bInIncludeChildren);

protected:
	//~ Begin FActorModifierCoreExtension
	ACTORMODIFIER_API virtual void OnExtensionEnabled(EActorModifierCoreEnableReason InReason) override;
	ACTORMODIFIER_API virtual void OnExtensionDisabled(EActorModifierCoreDisableReason InReason) override;
	//~ End FActorModifierCoreExtension

private:
	void OnRenderStateDirty(UActorComponent& InComponent);

	void BindDelegate();
	void UnbindDelegate();

	TWeakInterfacePtr<IActorModifierRenderStateUpdateHandler> ExtensionHandlerWeak;

	TMap<TWeakObjectPtr<AActor>, bool> TrackedActorsVisibility;

	/** The render state dirty reasons that are relevant to this extension */
	UE::ActorModifierCore::ERenderStateDirtyReason RelevantReasons;
};
