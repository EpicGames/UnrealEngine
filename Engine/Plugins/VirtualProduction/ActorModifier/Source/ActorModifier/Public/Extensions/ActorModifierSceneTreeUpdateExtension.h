// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modifiers/ActorModifierCoreExtension.h"
#include "UObject/WeakInterfacePtr.h"
#include "ActorModifierSceneTreeUpdateExtension.generated.h"

class ULevel;

/** Specifies the method for finding a reference actor based on it's position in the parent's hierarchy. */
UENUM(BlueprintType)
enum class EActorModifierReferenceContainer : uint8
{
	/** Uses the previous actor in the parent's hierarchy. */
	Previous,
	/** Uses the next actor in the parent's hierarchy. */
	Next,
	/** Uses the first actor in the parent's hierarchy. */
	First,
	/** Uses the last actor in the parent's hierarchy. */
	Last,
	/** Uses a specified reference actor set by the user. */
	Other
};

USTRUCT(BlueprintType)
struct FActorModifierSceneTreeActor
{
	friend class FActorModifierSceneTreeUpdateExtension;

	GENERATED_BODY()

	FActorModifierSceneTreeActor() = default;

	explicit FActorModifierSceneTreeActor(AActor* InActor)
		: ReferenceContainer(EActorModifierReferenceContainer::Other)
		, ReferenceActorWeak(InActor)
	{}

	/** The method for finding a reference actor based on it's position in the parent's hierarchy */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Outliner")
	EActorModifierReferenceContainer ReferenceContainer = EActorModifierReferenceContainer::Other;

	/** The actor being followed by the modifier. This is user selectable if the Reference Container is set to "Other" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Outliner", meta=(DisplayName="Reference Actor", EditCondition = "ReferenceContainer == EActorModifierReferenceContainer::Other"))
	TWeakObjectPtr<AActor> ReferenceActorWeak = nullptr;

	/** If true, will search for the next visible actor based on the selected reference container */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Outliner", meta=(EditCondition="ReferenceContainer != EActorModifierReferenceContainer::Other", EditConditionHides))
	bool bSkipHiddenActors = false;

	bool operator==(const FActorModifierSceneTreeActor& InOther) const
	{
		return ReferenceContainer == InOther.ReferenceContainer
			&& ReferenceActorWeak == InOther.ReferenceActorWeak
			&& bSkipHiddenActors == InOther.bSkipHiddenActors;
	}

	AActor* GetLocalActor() const
	{
		return LocalActorWeak.Get();
	}

protected:
	/** All children of reference actor to compare with new set children for changes */
	UPROPERTY(Transient)
	TSet<TWeakObjectPtr<AActor>> ReferenceActorChildrenWeak;

	/** Direct children of reference actor where order counts */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<AActor>> ReferenceActorDirectChildrenWeak;

	/** Tracked references actors, if we skip hidden actors, we still need to track those for visibility changes, can be rebuilt */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<AActor>> ReferenceActorsWeak;

	/** Parents of the reference actor */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<AActor>> ReferenceActorParentsWeak;

	/** Actor from which we start resolving this reference actor */
	UPROPERTY(Transient)
	TWeakObjectPtr<AActor> LocalActorWeak = nullptr;
};

UINTERFACE(MinimalAPI, NotBlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UActorModifierSceneTreeUpdateHandler : public UInterface
{
	GENERATED_BODY()
};

/** Implement this interface to handle extension event */
class IActorModifierSceneTreeUpdateHandler
{
	GENERATED_BODY()

public:
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) = 0;

	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) = 0;

	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) = 0;

	virtual void OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor) = 0;

	virtual void OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor) = 0;
};

/** Helps resolve underlying actors in the scene */
class IActorModifierSceneTreeCustomResolver : public TSharedFromThis<IActorModifierSceneTreeCustomResolver>
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorHierarchyChanged, AActor* /** InActor */)
	
	virtual ~IActorModifierSceneTreeCustomResolver() = default;
	
	virtual void Activate() = 0;
	virtual void Deactivate() = 0;
	virtual bool GetDirectChildrenActor(AActor* InActor, TArray<AActor*>& OutActors) const = 0;
	virtual bool GetRootActors(ULevel* InLevel, TArray<AActor*>& OutActors) const = 0;
	virtual FOnActorHierarchyChanged::RegistrationType& OnActorHierarchyChanged() = 0;
};

/**
 * This extension tracks specific actors for render state updates,
 * when an update happens it will dirty the modifier it is attached on if filter passes
 */
class FActorModifierSceneTreeUpdateExtension : public FActorModifierCoreExtension
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IActorModifierSceneTreeCustomResolver>, FOnGetSceneTreeResolver, ULevel* /** Level */)
	static FOnGetSceneTreeResolver::RegistrationType& OnGetSceneTreeResolver()
	{
		return OnGetSceneTreeResolverDelegate;
	}
	
	ACTORMODIFIER_API explicit FActorModifierSceneTreeUpdateExtension(IActorModifierSceneTreeUpdateHandler* InExtensionHandler);

	ACTORMODIFIER_API void TrackSceneTree(int32 InTrackedActorIdx, FActorModifierSceneTreeActor* InTrackedActor);
	ACTORMODIFIER_API void UntrackSceneTree(int32 InTrackedActorIdx);
	ACTORMODIFIER_API FActorModifierSceneTreeActor* GetTrackedActor(int32 InTrackedActorIdx) const;
	
	ACTORMODIFIER_API void CheckTrackedActorsUpdate() const;
	ACTORMODIFIER_API void CheckTrackedActorUpdate(int32 InIdx) const;
	
	ACTORMODIFIER_API TArray<TWeakObjectPtr<AActor>> GetDirectChildrenActor(AActor* InActor) const;

protected:
	//~ Begin FActorModifierCoreExtension
	ACTORMODIFIER_API virtual void OnExtensionInitialized() override;
	ACTORMODIFIER_API virtual void OnExtensionEnabled(EActorModifierCoreEnableReason InReason) override;
	ACTORMODIFIER_API virtual void OnExtensionDisabled(EActorModifierCoreDisableReason InReason) override;
	//~ End FActorModifierCoreExtension

	TSet<TWeakObjectPtr<AActor>> GetChildrenActorsRecursive(const AActor* InActor) const;
	TArray<TWeakObjectPtr<AActor>> GetParentActors(const AActor* InActor) const;
	TArray<TWeakObjectPtr<AActor>> GetReferenceActors(const FActorModifierSceneTreeActor* InReferenceActor) const;
	TArray<TWeakObjectPtr<AActor>> GetRootActors(ULevel* InLevel) const;

private:
	ACTORMODIFIER_API static FOnGetSceneTreeResolver OnGetSceneTreeResolverDelegate;

	void OnRefreshTrackedActors(AActor* InActor);

	void OnRenderStateDirty(UActorComponent& InComponent);
	void OnWorldActorDestroyed(AActor* InActor);

	bool IsSameActorArray(const TArray<TWeakObjectPtr<AActor>>& InPreviousActorWeak, const TArray<TWeakObjectPtr<AActor>>& InNewActorWeak) const;

	TWeakInterfacePtr<IActorModifierSceneTreeUpdateHandler> ExtensionHandlerWeak;

	TMap<int32, FActorModifierSceneTreeActor*> TrackedActors;

	FDelegateHandle WorldActorDestroyedDelegate;

	TSharedPtr<IActorModifierSceneTreeCustomResolver> SceneTreeResolver;
};
