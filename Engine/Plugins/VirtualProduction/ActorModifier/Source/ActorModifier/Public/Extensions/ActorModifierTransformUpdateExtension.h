// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreExtension.h"
#include "UObject/WeakInterfacePtr.h"
#include "ActorModifierTransformUpdateExtension.generated.h"

UINTERFACE(MinimalAPI, NotBlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UActorModifierTransformUpdateHandler : public UInterface
{
	GENERATED_BODY()
};

/** Implement this interface to handle extension event */
class IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

public:
	/** Callback when a tracked actor transform changes */
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) = 0;
};

/**
 * This extension tracks specific actors for transform updates,
 * when an update happens it will invoke the IAvaTransformUpdateExtension function
 */
class FActorModifierTransformUpdateExtension : public FActorModifierCoreExtension
{

public:
	ACTORMODIFIER_API explicit FActorModifierTransformUpdateExtension(IActorModifierTransformUpdateHandler* InExtensionHandler);

	ACTORMODIFIER_API void TrackActor(AActor* InActor, bool bInReset);
	ACTORMODIFIER_API void UntrackActor(AActor* InActor);

	ACTORMODIFIER_API void TrackActors(const TSet<TWeakObjectPtr<AActor>>& InActors, bool bInReset);
	ACTORMODIFIER_API void UntrackActors(const TSet<TWeakObjectPtr<AActor>>& InActors);

protected:
	//~ Begin FActorModifierCoreExtension
	ACTORMODIFIER_API virtual void OnExtensionEnabled(EActorModifierCoreEnableReason InReason) override;
	ACTORMODIFIER_API virtual void OnExtensionDisabled(EActorModifierCoreDisableReason InReason) override;
	//~ End FActorModifierCoreExtension

private:
	void OnTransformUpdated(USceneComponent* InComponent, EUpdateTransformFlags InFlags, ETeleportType InType);

	void BindDelegate(const AActor* InActor);
	void UnbindDelegate(const AActor* InActor);

	TWeakInterfacePtr<IActorModifierTransformUpdateHandler> ExtensionHandlerWeak;

	TSet<TWeakObjectPtr<AActor>> TrackedActors;
};