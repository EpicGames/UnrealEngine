// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Modifiers/ActorModifierAttachmentBaseModifier.h"
#include "ActorModifierArrangeBaseModifier.generated.h"

class AActor;

/**
 * Abstract base class for modifiers dealing with arrangement and attachment actors on self
 */
UCLASS(MinimalAPI, Abstract)
class UActorModifierArrangeBaseModifier : public UActorModifierAttachmentBaseModifier
	, public IActorModifierRenderStateUpdateHandler
	, public IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

protected:
	//~ Begin UActorModifierCoreBase
	ACTORMODIFIER_API virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	ACTORMODIFIER_API virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	ACTORMODIFIER_API virtual void OnModifiedActorTransformed() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	ACTORMODIFIER_API virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	ACTORMODIFIER_API virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override {}
	virtual void OnActorVisibilityChanged(AActor* InActor) override {}
	//~ End IAvaRenderStateUpdateExtension

	//~ Begin IAvaTransformUpdateExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override {}
	//~ End IAvaTransformUpdateExtension

	/** Used to track self modified actor for changes */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FActorModifierSceneTreeActor ReferenceActor;
};
