// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "ActorModifierAttachmentBaseModifier.generated.h"

class AActor;

/**
 * Abstract base class for all modifiers that deal with attachments
 */
UCLASS(MinimalAPI, Abstract)
class UActorModifierAttachmentBaseModifier : public UActorModifierCoreBase
	, public IActorModifierSceneTreeUpdateHandler
{
	GENERATED_BODY()

protected:
	//~ Begin UActorModifierCoreBase
	ACTORMODIFIER_API virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	ACTORMODIFIER_API virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) override {}
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override {}
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override {}
	virtual void OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor) override {}
	virtual void OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor) override {}
	//~ End IAvaSceneTreeUpdateModifierExtension

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TSet<TWeakObjectPtr<AActor>> ChildrenActorsWeak;
};
