// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ActorModifierTransformShared.generated.h"

class AActor;

/** Enumerates all transform state that can be saved */
UENUM()
enum class EActorModifierTransformSharedState : uint8
{
	None,
	Location = 1 << 0,
	Rotation = 1 << 1,
	Scale = 1 << 2,
	LocationRotation = Location | Rotation,
	LocationScale = Location | Scale,
	RotationScale = Rotation | Scale,
	All = Location | Rotation | Scale
};

USTRUCT()
struct FActorModifierTransformSharedModifierState
{
	GENERATED_BODY()

	FActorModifierTransformSharedModifierState() {}
	FActorModifierTransformSharedModifierState(UActorModifierCoreBase* InModifier)
		: ModifierWeak(InModifier)
	{}

	/** Save this modifier state if valid */
	void Save(const AActor* InActor, EActorModifierTransformSharedState InSaveState);

	/** Restore this modifier state if valid */
	void Restore(AActor* InActor, EActorModifierTransformSharedState InRestoreState);

	friend uint32 GetTypeHash(const FActorModifierTransformSharedModifierState& InItem)
	{
		return GetTypeHash(InItem.ModifierWeak);
	}

	bool operator==(const FActorModifierTransformSharedModifierState& Other) const
	{
		return ModifierWeak == Other.ModifierWeak;
	}

	/** Modifier applying the transform change */
	UPROPERTY()
	TWeakObjectPtr<UActorModifierCoreBase> ModifierWeak;

	/** Pre modifier transform saved */
	UPROPERTY()
	FTransform ActorTransform;

	/** Used to restore only what has changed */
	UPROPERTY()
	EActorModifierTransformSharedState SaveState = EActorModifierTransformSharedState::None;
};

USTRUCT()
struct FActorModifierTransformSharedActorState
{
	GENERATED_BODY()

	FActorModifierTransformSharedActorState() {}
	FActorModifierTransformSharedActorState(AActor* InActor)
		: ActorWeak(InActor)
	{}

	/** Save this actor state if valid */
	void Save(EActorModifierTransformSharedState InSaveState);

	/** Restore this actor state if valid */
	void Restore(EActorModifierTransformSharedState InRestoreState);

	friend uint32 GetTypeHash(const FActorModifierTransformSharedActorState& InItem)
	{
		return GetTypeHash(InItem.ActorWeak);
	}

	bool operator==(const FActorModifierTransformSharedActorState& Other) const
	{
		return ActorWeak == Other.ActorWeak;
	}

	/** Modifiers that are currently watching this state and locking it */
	UPROPERTY()
	TSet<FActorModifierTransformSharedModifierState> ModifierStates;

	/** Actor that this state is describing */
	UPROPERTY()
	TWeakObjectPtr<AActor> ActorWeak;

	/** Pre state transform saved */
	UPROPERTY()
	FTransform ActorTransform;

	/** Used to restore only what has changed */
	UPROPERTY()
	EActorModifierTransformSharedState SaveState = EActorModifierTransformSharedState::None;
};

/**
 * Singleton class for transform modifiers to share data between each other
 * Used because multiple modifier could be watching/updating an actor
 * We want to save the state of that actor once before any modifier changes it
 * and restore it when no other modifier is watching it
 */
UCLASS(Hidden, MinimalAPI)
class UActorModifierTransformShared : public UActorModifierCoreSharedObject
{
	GENERATED_BODY()

public:
	/** Save actor state, adds it if it is not tracked */
	ACTORMODIFIER_API void SaveActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor, EActorModifierTransformSharedState InSaveState = EActorModifierTransformSharedState::All);

	/** Restore actor state, removes it if no other modifier track that actor state */
	ACTORMODIFIER_API void RestoreActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor, EActorModifierTransformSharedState InRestoreState = EActorModifierTransformSharedState::All);

	/** Get the actor state for a specific actor */
	ACTORMODIFIER_API FActorModifierTransformSharedActorState* FindActorState(AActor* InActor);

	/** Get all actor state related to a modifier */
	ACTORMODIFIER_API TSet<FActorModifierTransformSharedActorState*> FindActorsState(UActorModifierCoreBase* InModifierContext);

	/** Restore all actors states linked to this modifier */
	ACTORMODIFIER_API void RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<AActor*>* InActors = nullptr, EActorModifierTransformSharedState InRestoreState = EActorModifierTransformSharedState::All);

	/** Restore all specified actors linked to this modifier */
	ACTORMODIFIER_API void RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<TWeakObjectPtr<AActor>>& InActors, EActorModifierTransformSharedState InRestoreState = EActorModifierTransformSharedState::All);

	/** Returns true, if this modifier is tracking this actor */
	ACTORMODIFIER_API bool IsActorStateSaved(UActorModifierCoreBase* InModifierContext, AActor* InActor);

	/** Returns true, if this modifier is tracking any actor */
	ACTORMODIFIER_API bool IsActorsStateSaved(UActorModifierCoreBase* InModifierContext);

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	/** Actor state before any modifier applied to it */
	UPROPERTY()
	TSet<FActorModifierTransformSharedActorState> ActorStates;
};
