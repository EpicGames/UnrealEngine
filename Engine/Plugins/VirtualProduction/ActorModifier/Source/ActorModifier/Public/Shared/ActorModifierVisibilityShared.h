// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ActorModifierVisibilityShared.generated.h"

class AActor;

UENUM(meta=(Bitflags))
enum class EActorModifierVisibilityActor : uint8
{
	None          = 0,
	Game          = 1 << 0,
	Editor        = 1 << 1,
	GameAndEditor = Game | Editor
};
ENUM_CLASS_FLAGS(EActorModifierVisibilityActor);

USTRUCT()
struct FActorModifierVisibilitySharedModifierState
{
	GENERATED_BODY()

	FActorModifierVisibilitySharedModifierState() {}
	FActorModifierVisibilitySharedModifierState(UActorModifierCoreBase* InModifier)
		: ModifierWeak(InModifier)
	{}

	/** Save this modifier state if valid */
	void Save(const AActor* InActor);

	/** Restore this modifier state if valid */
	void Restore(AActor* InActor) const;

	friend uint32 GetTypeHash(const FActorModifierVisibilitySharedModifierState& InItem)
	{
		return GetTypeHash(InItem.ModifierWeak);
	}

	bool operator==(const FActorModifierVisibilitySharedModifierState& Other) const
	{
		return ModifierWeak == Other.ModifierWeak;
	}

	UPROPERTY()
	TWeakObjectPtr<UActorModifierCoreBase> ModifierWeak;

#if WITH_EDITORONLY_DATA
	/** Pre state editor visibility saved */
	UPROPERTY()
	bool bActorHiddenInEditor = false;
#endif

	/** Pre state game visibility saved */
	UPROPERTY()
	bool bActorHiddenInGame = false;
};

USTRUCT()
struct FActorModifierVisibilitySharedActorState
{
	GENERATED_BODY()

	FActorModifierVisibilitySharedActorState() {}
	FActorModifierVisibilitySharedActorState(AActor* InActor)
		: ActorWeak(InActor)
	{}

	/** Save this actor state if valid */
	void Save();

	/** Restore this actor state if valid */
	void Restore() const;

	friend uint32 GetTypeHash(const FActorModifierVisibilitySharedActorState& InItem)
	{
		return GetTypeHash(InItem.ActorWeak);
	}

	bool operator==(const FActorModifierVisibilitySharedActorState& Other) const
	{
		return ActorWeak == Other.ActorWeak;
	}

	/** Modifiers that are currently watching this state and locking it */
	UPROPERTY()
	TSet<FActorModifierVisibilitySharedModifierState> ModifierStates;

	/** Actor that this state is describing */
	UPROPERTY()
	TWeakObjectPtr<AActor> ActorWeak;

#if WITH_EDITORONLY_DATA
	/** Pre state editor visibility saved */
	UPROPERTY()
	bool bActorHiddenInEditor = false;
#endif

	/** Pre state game visibility saved */
	UPROPERTY()
	bool bActorHiddenInGame = false;
};

/**
 * Singleton class for visibility modifiers to share data between each other
 * Used because multiple modifier could be watching/updating an actor
 * We want to save the state of that actor once before any modifier changes it
 * and restore it when no other modifier is watching it
 */
UCLASS(Hidden, MinimalAPI)
class UActorModifierVisibilityShared : public UActorModifierCoreSharedObject
{
	GENERATED_BODY()

public:
	/** Watch actor state, adds it if it is not tracked */
	ACTORMODIFIER_API void SaveActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor);

	/** Unwatch actor state, removes it if no other modifier track that actor state */
	ACTORMODIFIER_API void RestoreActorState(UActorModifierCoreBase* InModifierContext, AActor* InActor);

	/** Gather original state before any modifier is applied if there is one */
	ACTORMODIFIER_API FActorModifierVisibilitySharedActorState* FindActorState(AActor* InActor);

	/** Set actor visibility in game or editor and recurse, tracks original state if not tracked */
	ACTORMODIFIER_API void SetActorVisibility(UActorModifierCoreBase* InModifierContext, AActor* InActor, bool bInHidden, bool bInRecurse = false, EActorModifierVisibilityActor InActorVisibility = EActorModifierVisibilityActor::GameAndEditor);

	/** Set actors visibility in game or editor, tracks original state if not tracked */
	void SetActorsVisibility(UActorModifierCoreBase* InModifierContext, TArray<AActor*> InActors, bool bInHidden, EActorModifierVisibilityActor InActorVisibility = EActorModifierVisibilityActor::GameAndEditor);

	/** Unwatch all actors states linked to this modifier */
	ACTORMODIFIER_API void RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<AActor*>* InActors = nullptr);

	ACTORMODIFIER_API void RestoreActorsState(UActorModifierCoreBase* InModifierContext, const TSet<TWeakObjectPtr<AActor>>& InActors);

	/** Returns true, if this modifier is tracking this actor */
	ACTORMODIFIER_API bool IsActorStateSaved(UActorModifierCoreBase* InModifierContext, AActor* InActor);

	/** Returns true, if this modifier is tracking any actor */
	bool IsActorsStateSaved(UActorModifierCoreBase* InModifierContext);

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	/** Actor state before any modifier applied to it */
	UPROPERTY()
	TSet<FActorModifierVisibilitySharedActorState> ActorStates;
};
