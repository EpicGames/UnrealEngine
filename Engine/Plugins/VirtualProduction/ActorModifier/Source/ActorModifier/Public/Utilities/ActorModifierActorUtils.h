// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierTypes.h"
#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/OrientedBox.h"
#include "UObject/WeakObjectPtrFwd.h"

class AActor;

/** Operations that can be reused or shared within modifiers */
namespace UE::ActorModifier::ActorUtils
{
	// Rendering
	ACTORMODIFIER_API bool IsActorVisible(const AActor* InActor);

	// Layout
	ACTORMODIFIER_API FBox GetActorsBounds(const TSet<TWeakObjectPtr<AActor>>& InActors, const FTransform& InReferenceTransform, bool bInSkipHidden, bool bInTransformBox);
	ACTORMODIFIER_API FBox GetActorsBounds(const TSet<TWeakObjectPtr<AActor>>& InActors, const FTransform& InReferenceTransform, bool bInSkipHidden = false);
	ACTORMODIFIER_API FBox GetActorsBounds(AActor* InActor, bool bInIncludeChildren, bool bInSkipHidden = false);
	ACTORMODIFIER_API FBox GetActorBounds(const AActor* InActor);
	ACTORMODIFIER_API FVector GetVectorAxis(int32 InAxis);
	ACTORMODIFIER_API bool IsAxisVectorEquals(const FVector& InVectorA, const FVector& InVectorB, int32 InCompareAxis);
	ACTORMODIFIER_API FOrientedBox GetOrientedBox(const FBox& InLocalBox, const FTransform& InWorldTransform);
	ACTORMODIFIER_API FRotator FindLookAtRotation(const FVector& InEyePosition, const FVector& InTargetPosition, EActorModifierAxis InAxis, bool bInFlipAxis);
};