// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierEditor : ModuleRules
{
    public ActorModifierEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "ActorModifierCore",
	            "ActorModifier",
				"ActorModifierLayout",
				"ClonerEffectorEditor",
                "CoreUObject",
                "Engine",
                "Projects",
                "Slate",
                "SlateCore"
            }
        );
    }
}