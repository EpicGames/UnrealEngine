// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorModifierEditorModule.h"

#include "ActorModifierTypes.h"
#include "Customizations/ActorModifierEditorActorComponentClassPropertyTypeCustomization.h"
#include "Customizations/ActorModifierEditorAnchorAlignmentPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "Styles/ActorModifierEditorStyle.h"

IMPLEMENT_MODULE(FActorModifierEditorModule, ActorModifierEditor)

void FActorModifierEditorModule::StartupModule()
{
	using namespace UE::ActorModifierEditor::Private;

	FActorModifierEditorStyle::Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(FActorModifierAnchorAlignment::StaticStruct()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(RegisteredCustomizations.Add_GetRef(FWeakObjectProperty::StaticClass()->GetFName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FActorModifierEditorActorComponentClassPropertyTypeCustomization::MakeInstance), MakeShared<FActorModifierEditorActorComponentClassPropertyTypeIdentifier>());
}

void FActorModifierEditorModule::ShutdownModule()
{
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(TEXT("PropertyEditor")))
	{
		for (const FName& Name : RegisteredCustomizations)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(Name);
		}
	}
}
