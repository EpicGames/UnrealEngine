// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/ActorModifierEditorActorComponentClassPropertyTypeCustomization.h"

#include "Cloner/Customizations/CEEditorClonerCustomActorPickerNodeBuilder.h"
#include "Components/ActorComponent.h"
#include "DetailWidgetRow.h"
#include "GameFramework/Actor.h"

namespace UE::ActorModifierEditor::Private
{
	const FLazyName FActorModifierEditorActorComponentClassPropertyTypeIdentifier::MetadataSpecifier = TEXT("FilterActorByComponentClass");

	bool FActorModifierEditorActorComponentClassPropertyTypeIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const
	{
		if (InPropertyHandle.HasMetaData(MetadataSpecifier))
		{
			if (const FWeakObjectProperty* ObjectProperty = CastField<FWeakObjectProperty>(InPropertyHandle.GetProperty()))
			{
				return ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(AActor::StaticClass());
			}
		}

		return false;
	}

	TSharedRef<IPropertyTypeCustomization> FActorModifierEditorActorComponentClassPropertyTypeCustomization::MakeInstance()
	{
		return MakeShared<FActorModifierEditorActorComponentClassPropertyTypeCustomization>();
	}

	void FActorModifierEditorActorComponentClassPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InUtils)
	{
		InPropertyHandle->MarkHiddenByCustomization();

		ComponentClass = InPropertyHandle->GetClassMetaData(FActorModifierEditorActorComponentClassPropertyTypeIdentifier::MetadataSpecifier);

		ActorPickerNodeBuilder = MakeShared<FCEEditorClonerCustomActorPickerNodeBuilder>(
			InPropertyHandle
			, FOnShouldFilterActor::CreateSP(this, &FActorModifierEditorActorComponentClassPropertyTypeCustomization::OnFilterActorByComponentClass)
		);

		ActorPickerNodeBuilder->GenerateHeaderRowContent(InHeaderRow);
	}

	void FActorModifierEditorActorComponentClassPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils)
	{
		ActorPickerNodeBuilder->GenerateChildContent(InBuilder);
	}

	bool FActorModifierEditorActorComponentClassPropertyTypeCustomization::OnFilterActorByComponentClass(const AActor* InActor)
	{
		if (ComponentClass.Get())
		{
			return !!InActor->FindComponentByClass(ComponentClass);
		}

		return true;
	}
}