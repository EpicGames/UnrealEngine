// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "Templates/SubclassOf.h"

class FCEEditorClonerCustomActorPickerNodeBuilder;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class UActorComponent;

namespace UE::ActorModifierEditor::Private
{
	/**
	 * Only allow property customization with class metadata specifier
	 */
	class FActorModifierEditorActorComponentClassPropertyTypeIdentifier : public IPropertyTypeIdentifier
	{
	public:
		static const FLazyName MetadataSpecifier;

		//~ Begin IPropertyTypeIdentifier
		virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override;
		//~ End IPropertyTypeIdentifier
	};

	/*
	 * Customization for an actor property to only show filtered actor based on component class
	*/
	class FActorModifierEditorActorComponentClassPropertyTypeCustomization : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin IPropertyTypeCustomization
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InBuilder, IPropertyTypeCustomizationUtils& InUtils) override;
		//~ End IPropertyTypeCustomization

	protected:
		bool OnFilterActorByComponentClass(const AActor* InActor);

		TSubclassOf<UActorComponent> ComponentClass;
		TSharedPtr<FCEEditorClonerCustomActorPickerNodeBuilder> ActorPickerNodeBuilder;
	};
}