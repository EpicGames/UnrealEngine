// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/ActorModifierEditorAnchorAlignmentPropertyTypeCustomization.h"

#include "Customizations/SActorModifierEditorAnchorAlignment.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "ActorModifierEditorAnchorAlignmentPropertyTypeCustomization"

TSharedRef<IPropertyTypeCustomization> FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FActorModifierEditorAnchorAlignmentPropertyTypeCustomization>();
}

void FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle
	, FDetailWidgetRow& HeaderRow
	, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	check(StructPropertyHandle.IsValid());

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SActorModifierEditorAnchorAlignment)
			.UniformPadding(FMargin(5.0f, 2.0f))
			.Anchors(this, &FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::GetAnchors)
			.OnAnchorChanged(this, &FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::OnAnchorChanged)
		];
}

FActorModifierAnchorAlignment FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::GetAnchors() const
{
	if (!StructPropertyHandle.IsValid())
	{
		return FActorModifierAnchorAlignment();
	}
	
	void* AnchorsPtr = nullptr;
	StructPropertyHandle->GetValueData(AnchorsPtr);
	const FActorModifierAnchorAlignment* Anchors = static_cast<FActorModifierAnchorAlignment*>(AnchorsPtr);
	
	if (!Anchors)
	{
		return FActorModifierAnchorAlignment();
	}
	
	return *Anchors;
}

void FActorModifierEditorAnchorAlignmentPropertyTypeCustomization::OnAnchorChanged(const FActorModifierAnchorAlignment NewAnchor) const
{
	if (!StructPropertyHandle.IsValid())
	{
		return;
	}
	
	void* AnchorsPtr = nullptr;
	StructPropertyHandle->GetValueData(AnchorsPtr);
	FActorModifierAnchorAlignment* Anchors = static_cast<FActorModifierAnchorAlignment*>(AnchorsPtr);

	if (!Anchors)
	{
		return;
	}

	StructPropertyHandle->NotifyPreChange();

	*Anchors = NewAnchor;

	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

#undef LOCTEXT_NAMESPACE
