// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyEditorModule.h"
#include "IPropertyTypeCustomization.h"

struct FActorModifierAnchorAlignment;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

/**
 * Motion Design Anchor Alignment Customization
 * 
 * Shows a grid of buttons for setting anchor alignment in the following layout:
 * [HLeft ] [HCenter] [HRight ]
 * [VTop  ] [VCenter] [VBottom]
 * [DFront] [DCenter] [DBack  ]
*/
class FActorModifierEditorAnchorAlignmentPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	//~ End IPropertyTypeCustomization

private:
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	FActorModifierAnchorAlignment GetAnchors() const;

	void OnAnchorChanged(const FActorModifierAnchorAlignment NewAnchor) const;
};
