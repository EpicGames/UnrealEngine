// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SActorModifierEditorAnchorAlignment.h"

#include "Customizations/SActorModifierEditorDepthAlignment.h"
#include "Customizations/SActorModifierEditorHorizontalAlignment.h"
#include "Customizations/SActorModifierEditorVerticalAlignment.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "ActorModifierEditorAnchorAlignment"

void SActorModifierEditorAnchorAlignment::Construct(const FArguments& InArgs)
{
	Anchors = InArgs._Anchors;

	OnAnchorChanged = InArgs._OnAnchorChanged;

	ChildSlot
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f, 0.0f, 1.0f)
		[
			SNew(SActorModifierEditorHorizontalAlignment)
			.UniformPadding(InArgs._UniformPadding)
			.Visibility(this, &SActorModifierEditorAnchorAlignment::GetHorizontalVisibility)
			.Alignment(this, &SActorModifierEditorAnchorAlignment::GetHorizontalAlignment)
			.OnAlignmentChanged(this, &SActorModifierEditorAnchorAlignment::OnHorizontalAlignmentChanged)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 1.0f)
		[
			SNew(SActorModifierEditorVerticalAlignment)
			.UniformPadding(InArgs._UniformPadding)
			.Visibility(this, &SActorModifierEditorAnchorAlignment::GetVerticalVisibility)
			.Alignment(this, &SActorModifierEditorAnchorAlignment::GetVerticalAlignment)
			.OnAlignmentChanged(this, &SActorModifierEditorAnchorAlignment::OnVerticalAlignmentChanged)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 2.0f)
		[
			SNew(SActorModifierEditorDepthAlignment)
			.UniformPadding(InArgs._UniformPadding)
			.Visibility(this, &SActorModifierEditorAnchorAlignment::GetDepthVisibility)
			.Alignment(this, &SActorModifierEditorAnchorAlignment::GetDepthAlignment)
			.OnAlignmentChanged(this, &SActorModifierEditorAnchorAlignment::OnDepthAlignmentChanged)
		]
	];
}

EVisibility SActorModifierEditorAnchorAlignment::GetHorizontalVisibility() const
{
	if (!Anchors.IsSet())
	{
		return EVisibility::SelfHitTestInvisible;
	}
	
	return Anchors.Get().bUseHorizontal ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; 
}

EVisibility SActorModifierEditorAnchorAlignment::GetVerticalVisibility() const
{
	if (!Anchors.IsSet())
	{
		return EVisibility::SelfHitTestInvisible;
	}
	
	return Anchors.Get().bUseVertical ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; 
}

EVisibility SActorModifierEditorAnchorAlignment::GetDepthVisibility() const
{
	if (!Anchors.IsSet())
	{
		return EVisibility::SelfHitTestInvisible;
	}
	
	return Anchors.Get().bUseDepth ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; 
}

EActorModifierHorizontalAlignment SActorModifierEditorAnchorAlignment::GetHorizontalAlignment() const
{
	return Anchors.IsSet() ? Anchors.Get().Horizontal : EActorModifierHorizontalAlignment::Center;
}

EActorModifierVerticalAlignment SActorModifierEditorAnchorAlignment::GetVerticalAlignment() const
{
	return Anchors.IsSet() ? Anchors.Get().Vertical : EActorModifierVerticalAlignment::Center;
}

EActorModifierDepthAlignment SActorModifierEditorAnchorAlignment::GetDepthAlignment() const
{
	return Anchors.IsSet() ? Anchors.Get().Depth : EActorModifierDepthAlignment::Center;
}

void SActorModifierEditorAnchorAlignment::OnHorizontalAlignmentChanged(const EActorModifierHorizontalAlignment InAlignment)
{
	if (!Anchors.IsSet())
	{
		return;
	}

	FActorModifierAnchorAlignment AnchorAlignment = Anchors.Get();
	AnchorAlignment.Horizontal = InAlignment;

	if (OnAnchorChanged.IsBound())
	{
		OnAnchorChanged.Execute(AnchorAlignment);
	}
}

void SActorModifierEditorAnchorAlignment::OnVerticalAlignmentChanged(const EActorModifierVerticalAlignment InAlignment)
{
	if (!Anchors.IsSet())
	{
		return;
	}

	FActorModifierAnchorAlignment AnchorAlignment = Anchors.Get();
	AnchorAlignment.Vertical = InAlignment;
	
	if (OnAnchorChanged.IsBound())
	{
		OnAnchorChanged.Execute(AnchorAlignment);
	}
}

void SActorModifierEditorAnchorAlignment::OnDepthAlignmentChanged(const EActorModifierDepthAlignment InAlignment)
{
	if (!Anchors.IsSet())
	{
		return;
	}

	FActorModifierAnchorAlignment AnchorAlignment = Anchors.Get();
	AnchorAlignment.Depth = InAlignment;
	
	if (OnAnchorChanged.IsBound())
	{
		OnAnchorChanged.Execute(AnchorAlignment);
	}
}

#undef LOCTEXT_NAMESPACE
