// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierTypes.h"
#include "Delegates/Delegate.h"
#include "Widgets/SCompoundWidget.h"

class SUniformGridPanel;

/**
 * Motion Design Anchor Alignment
 * 
 * Widget that holds holds axis alignment widgets for Left/Center/Right, Top/Center/Bottom, and Front/Center/Back.
 * One alignment button set per axis row (Horizontal, Vertical, Depth).
 */
class SActorModifierEditorAnchorAlignment : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnAnchorChanged, FActorModifierAnchorAlignment)

	SLATE_BEGIN_ARGS(SActorModifierEditorAnchorAlignment)
	{}
		SLATE_ATTRIBUTE(FActorModifierAnchorAlignment, Anchors)
		SLATE_ATTRIBUTE(FMargin, UniformPadding)
		SLATE_EVENT(FOnAnchorChanged, OnAnchorChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	EVisibility GetHorizontalVisibility() const;
	EVisibility GetVerticalVisibility() const;
	EVisibility GetDepthVisibility() const;

	EActorModifierHorizontalAlignment GetHorizontalAlignment() const;
	EActorModifierVerticalAlignment GetVerticalAlignment() const;
	EActorModifierDepthAlignment GetDepthAlignment() const;

	void OnHorizontalAlignmentChanged(const EActorModifierHorizontalAlignment InAlignment);
	void OnVerticalAlignmentChanged(const EActorModifierVerticalAlignment InAlignment);
	void OnDepthAlignmentChanged(const EActorModifierDepthAlignment InAlignment);

protected:
	TAttribute<FActorModifierAnchorAlignment> Anchors;

	FOnAnchorChanged OnAnchorChanged;
};
