// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SActorModifierEditorDepthAlignment.h"

#include "ActorModifierTypes.h"
#include "SlateOptMacros.h"
#include "Styles/ActorModifierEditorStyle.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "ActorModifierEditorDepthAlignment"

void SActorModifierEditorDepthAlignment::Construct(const FArguments& InArgs)
{
	Alignment = InArgs._Alignment;
	OnAlignmentChanged = InArgs._OnAlignmentChanged;

	ChildSlot
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SSegmentedControl<EActorModifierDepthAlignment>)
		.UniformPadding(InArgs._UniformPadding)
		.Value(this, &SActorModifierEditorDepthAlignment::GetCurrentAlignment)
		.OnValueChanged(this, &SActorModifierEditorDepthAlignment::OnCurrentAlignmentChanged)
		+ SSegmentedControl<EActorModifierDepthAlignment>::Slot(EActorModifierDepthAlignment::Front)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.DepthFront"))
			.ToolTip(LOCTEXT("DAlignFront", "Front Align Depth"))
		+ SSegmentedControl<EActorModifierDepthAlignment>::Slot(EActorModifierDepthAlignment::Center)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.DepthCenter"))
			.ToolTip(LOCTEXT("DAlignCenter", "Center Align Depth"))
		+ SSegmentedControl<EActorModifierDepthAlignment>::Slot(EActorModifierDepthAlignment::Back)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.DepthBack"))
			.ToolTip(LOCTEXT("DAlignBack", "Back Align Depth"))
	];
}

EActorModifierDepthAlignment SActorModifierEditorDepthAlignment::GetCurrentAlignment() const
{
	return Alignment.Get(EActorModifierDepthAlignment::Center);
}

void SActorModifierEditorDepthAlignment::OnCurrentAlignmentChanged(const EActorModifierDepthAlignment NewAlignment)
{
	OnAlignmentChanged.ExecuteIfBound(NewAlignment);
}

#undef LOCTEXT_NAMESPACE
