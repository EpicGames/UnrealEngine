// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Widgets/SCompoundWidget.h"

enum class EActorModifierDepthAlignment : uint8;

/**
 * Motion Design Depth Axis Alignment
 * 
 * Widget that holds three buttons for either Front, Center, and Back.
 */
class SActorModifierEditorDepthAlignment : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnDepthAlignmentChanged, EActorModifierDepthAlignment)

	SLATE_BEGIN_ARGS(SActorModifierEditorDepthAlignment)
	{}
		SLATE_ATTRIBUTE(EActorModifierDepthAlignment, Alignment)
		SLATE_ATTRIBUTE(FMargin, UniformPadding)
		SLATE_EVENT(FOnDepthAlignmentChanged, OnAlignmentChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	TAttribute<EActorModifierDepthAlignment> Alignment;

	FOnDepthAlignmentChanged OnAlignmentChanged;

	EActorModifierDepthAlignment GetCurrentAlignment() const;

	void OnCurrentAlignmentChanged(const EActorModifierDepthAlignment NewAlignment);
};
