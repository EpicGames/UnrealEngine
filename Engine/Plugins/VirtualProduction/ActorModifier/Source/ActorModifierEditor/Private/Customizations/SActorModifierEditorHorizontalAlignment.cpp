// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SActorModifierEditorHorizontalAlignment.h"

#include "ActorModifierTypes.h"
#include "SlateOptMacros.h"
#include "Styles/ActorModifierEditorStyle.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "ActorModifierEditorHorizontalAlignment"

void SActorModifierEditorHorizontalAlignment::Construct(const FArguments& InArgs)
{
	Alignment = InArgs._Alignment;
	OnAlignmentChanged = InArgs._OnAlignmentChanged;

	ChildSlot
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SSegmentedControl<EActorModifierHorizontalAlignment>)
		.UniformPadding(InArgs._UniformPadding)
		.Value(this, &SActorModifierEditorHorizontalAlignment::GetCurrentAlignment)
		.OnValueChanged(this, &SActorModifierEditorHorizontalAlignment::OnCurrentAlignmentChanged)
		+ SSegmentedControl<EActorModifierHorizontalAlignment>::Slot(EActorModifierHorizontalAlignment::Left)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.HorizontalLeft"))
			.ToolTip(LOCTEXT("HAlignLeft", "Left Align Horizontally"))
		+ SSegmentedControl<EActorModifierHorizontalAlignment>::Slot(EActorModifierHorizontalAlignment::Center)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.HorizontalCenter"))
			.ToolTip(LOCTEXT("HAlignCenter", "Center Align Horizontally"))
		+ SSegmentedControl<EActorModifierHorizontalAlignment>::Slot(EActorModifierHorizontalAlignment::Right)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.HorizontalRight"))
			.ToolTip(LOCTEXT("HAlignRight", "Right Align Horizontally"))
	];
}

EActorModifierHorizontalAlignment SActorModifierEditorHorizontalAlignment::GetCurrentAlignment() const
{
	return Alignment.Get(EActorModifierHorizontalAlignment::Center);
}

void SActorModifierEditorHorizontalAlignment::OnCurrentAlignmentChanged(const EActorModifierHorizontalAlignment NewAlignment)
{
	OnAlignmentChanged.ExecuteIfBound(NewAlignment);
}

#undef LOCTEXT_NAMESPACE
