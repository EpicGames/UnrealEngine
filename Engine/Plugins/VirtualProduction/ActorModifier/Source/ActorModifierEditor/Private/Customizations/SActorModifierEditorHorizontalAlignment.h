// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Widgets/SCompoundWidget.h"

enum class EActorModifierHorizontalAlignment : uint8;

/**
 * Avalanche Horizontal Axis Alignment
 * 
 * Widget that holds three buttons for either Left, Center, and Right.
 */
class SActorModifierEditorHorizontalAlignment : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnHorizontalAlignmentChanged, EActorModifierHorizontalAlignment)

	SLATE_BEGIN_ARGS(SActorModifierEditorHorizontalAlignment)
	{}
		SLATE_ATTRIBUTE(EActorModifierHorizontalAlignment, Alignment)
		SLATE_ATTRIBUTE(FMargin, UniformPadding)
		SLATE_EVENT(FOnHorizontalAlignmentChanged, OnAlignmentChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	TAttribute<EActorModifierHorizontalAlignment> Alignment;

	FOnHorizontalAlignmentChanged OnAlignmentChanged;

	EActorModifierHorizontalAlignment GetCurrentAlignment() const;

	void OnCurrentAlignmentChanged(const EActorModifierHorizontalAlignment NewAlignment);
};
