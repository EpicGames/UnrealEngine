// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SActorModifierEditorVerticalAlignment.h"

#include "ActorModifierTypes.h"
#include "SlateOptMacros.h"
#include "Styles/ActorModifierEditorStyle.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "ActorModifierEditorVerticalAlignment"

void SActorModifierEditorVerticalAlignment::Construct(const FArguments& InArgs)
{
	Alignment = InArgs._Alignment;
	OnAlignmentChanged = InArgs._OnAlignmentChanged;

	ChildSlot
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SSegmentedControl<EActorModifierVerticalAlignment>)
		.UniformPadding(InArgs._UniformPadding)
		.Value(this, &SActorModifierEditorVerticalAlignment::GetCurrentAlignment)
		.OnValueChanged(this, &SActorModifierEditorVerticalAlignment::OnCurrentAlignmentChanged)
		+ SSegmentedControl<EActorModifierVerticalAlignment>::Slot(EActorModifierVerticalAlignment::Top)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.VerticalTop"))
			.ToolTip(LOCTEXT("VAlignTop", "Top Align Vertically"))
		+ SSegmentedControl<EActorModifierVerticalAlignment>::Slot(EActorModifierVerticalAlignment::Center)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.VerticalCenter"))
			.ToolTip(LOCTEXT("VAlignCenter", "Center Align Vertically"))
		+ SSegmentedControl<EActorModifierVerticalAlignment>::Slot(EActorModifierVerticalAlignment::Bottom)
			.Icon(FActorModifierEditorStyle::Get().GetBrush("Icons.VerticalBottom"))
			.ToolTip(LOCTEXT("VAlignBottom", "Bottom Align Vertically"))
	];
}

EActorModifierVerticalAlignment SActorModifierEditorVerticalAlignment::GetCurrentAlignment() const
{
	return Alignment.Get(EActorModifierVerticalAlignment::Center);
}

void SActorModifierEditorVerticalAlignment::OnCurrentAlignmentChanged(const EActorModifierVerticalAlignment NewAlignment)
{
	OnAlignmentChanged.ExecuteIfBound(NewAlignment);
}

#undef LOCTEXT_NAMESPACE
