// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Widgets/SCompoundWidget.h"

enum class EActorModifierVerticalAlignment : uint8;

/**
 * Motion Design Vertical Axis Alignment
 * 
 * Widget that holds three buttons for either Top, Center, and Bottom.
 */
class SActorModifierEditorVerticalAlignment : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnVerticalAlignmentChanged, EActorModifierVerticalAlignment)

	SLATE_BEGIN_ARGS(SActorModifierEditorVerticalAlignment)
	{}
		SLATE_ATTRIBUTE(EActorModifierVerticalAlignment, Alignment)
		SLATE_ATTRIBUTE(FMargin, UniformPadding)
		SLATE_EVENT(FOnVerticalAlignmentChanged, OnAlignmentChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	TAttribute<EActorModifierVerticalAlignment> Alignment;
	FOnVerticalAlignmentChanged OnAlignmentChanged;

	EActorModifierVerticalAlignment GetCurrentAlignment() const;

	void OnCurrentAlignmentChanged(const EActorModifierVerticalAlignment NewAlignment);
};
