// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/ActorModifierEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

TMap<FName, FSlateColor> FActorModifierEditorStyle::ModifierCategoriesColors =
{
	{TEXT("Unspecified"), FSlateColor(FActorModifierCoreMetadata::DefaultColor)},
	{TEXT("Geometry"), FSlateColor(FLinearColor::Yellow.Desaturate(0.25))},
	{TEXT("Transform"), FSlateColor(FLinearColor::Blue.Desaturate(0.25))},
	{TEXT("Layout"), FSlateColor(FLinearColor::Green.Desaturate(0.25))},
	{TEXT("Rendering"), FSlateColor(FLinearColor::Red.Desaturate(0.25))}
};

FActorModifierEditorStyle::FActorModifierEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon16x16(16.f, 16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Content"));

	Set("ClassIcon.ActorModifierCoreBase", new IMAGE_BRUSH_SVG("Icons/Modifiers/BaseModifier", Icon16x16));
	
	Set("Icons.DepthBack",   new IMAGE_BRUSH_SVG("Icons/Alignments/DepthBack",   Icon16x16));
	Set("Icons.DepthCenter", new IMAGE_BRUSH_SVG("Icons/Alignments/DepthCenter", Icon16x16));
	Set("Icons.DepthFront",  new IMAGE_BRUSH_SVG("Icons/Alignments/DepthFront",  Icon16x16));

	Set("Icons.HorizontalLeft",   new IMAGE_BRUSH_SVG("Icons/Alignments/HorizontalLeft",   Icon16x16));
	Set("Icons.HorizontalCenter", new IMAGE_BRUSH_SVG("Icons/Alignments/HorizontalCenter", Icon16x16));
	Set("Icons.HorizontalRight",  new IMAGE_BRUSH_SVG("Icons/Alignments/HorizontalRight",  Icon16x16));
	
	Set("Icons.VerticalTop",    new IMAGE_BRUSH_SVG("Icons/Alignments/VerticalTop",    Icon16x16));
	Set("Icons.VerticalCenter", new IMAGE_BRUSH_SVG("Icons/Alignments/VerticalCenter", Icon16x16));
	Set("Icons.VerticalBottom", new IMAGE_BRUSH_SVG("Icons/Alignments/VerticalBottom", Icon16x16));

	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		// loop through already registered factories if subsystem exists
		ModifierSubsystem->ForEachModifierMetadata([this](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			OnModifierClassRegistered(InMetadata);
			return true;
		});
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);

	UActorModifierCoreSubsystem::OnModifierClassRegistered().AddRaw(this, &FActorModifierEditorStyle::OnModifierClassRegistered);
}

const FSlateColor& FActorModifierEditorStyle::GetModifierCategoryColor(FName CategoryName)
{
	if (ModifierCategoriesColors.Contains(CategoryName))
	{
		return ModifierCategoriesColors[CategoryName];
	}

	return ModifierCategoriesColors[TEXT("Unspecified")];
}

void FActorModifierEditorStyle::OnModifierClassRegistered(const FActorModifierCoreMetadata& InMetadata)
{
	// We need to update the metadata to set various options
	FActorModifierCoreMetadata& MutableMetadata = const_cast<FActorModifierCoreMetadata&>(InMetadata);

	const FVector2f Icon16x16(16.f, 16.f);
	const FString ModifierIconPath = TEXT("Icons/Modifiers/") + InMetadata.GetName().ToString() + TEXT("Modifier");

	MutableMetadata.SetColor(GetModifierCategoryColor(InMetadata.GetCategory()).GetSpecifiedColor());

	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*RootToContentDir(ModifierIconPath, TEXT(".svg"))))
	{
		const FName ModifierClassStyleName(TEXT("ClassIcon.") + InMetadata.GetClass()->GetName());
		Set(ModifierClassStyleName, new IMAGE_BRUSH_SVG(ModifierIconPath, Icon16x16));
		MutableMetadata.SetIcon(FSlateIcon(GetStyleSetName(), ModifierClassStyleName));
	}
}

FActorModifierEditorStyle::~FActorModifierEditorStyle()
{
	UActorModifierCoreSubsystem::OnModifierClassRegistered().RemoveAll(this);

	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
