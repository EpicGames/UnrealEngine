// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Styling/SlateStyle.h"

struct FActorModifierCoreMetadata;

class FActorModifierEditorStyle final : public FSlateStyleSet
{
public:
	static FActorModifierEditorStyle& Get()
	{
		static FActorModifierEditorStyle Instance;
		return Instance;
	}

	FActorModifierEditorStyle();
	virtual ~FActorModifierEditorStyle() override;

	ACTORMODIFIEREDITOR_API static const FSlateColor& GetModifierCategoryColor(FName CategoryName);

private:
	void OnModifierClassRegistered(const FActorModifierCoreMetadata& InMetadata);

	static TMap<FName, FSlateColor> ModifierCategoriesColors;
};
