// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierLayout : ModuleRules
{
    public ActorModifierLayout(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
	            "ActorModifier",
	            "ActorModifierCore",
                "Core", 
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine"
            }
        );
    }
}