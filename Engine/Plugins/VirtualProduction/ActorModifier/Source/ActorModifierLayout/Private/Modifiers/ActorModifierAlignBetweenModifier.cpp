// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierAlignBetweenModifier.h"

#include "GameFramework/Actor.h"
#include "Misc/ITransaction.h"
#include "Shared/ActorModifierTransformShared.h"

#define LOCTEXT_NAMESPACE "ActorModifierAlignBetweenModifier"

void UActorModifierAlignBetweenModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierTransformUpdateExtension>(this);
}

void UActorModifierAlignBetweenModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	// Save actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(true))
	{
		LayoutShared->SaveActorState(this, GetModifiedActor(), EActorModifierTransformSharedState::Location);
	}

	SetTransformExtensionReferenceActors();
}

void UActorModifierAlignBetweenModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	// Restore actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(false))
	{
		LayoutShared->RestoreActorState(this, GetModifiedActor(), EActorModifierTransformSharedState::Location);
	}
}

void UActorModifierAlignBetweenModifier::Apply()
{
	AActor* const ModifyActor = GetModifiedActor();
	
	if (ReferenceActors.IsEmpty())
	{
		Next();
		return;
	}

	const TSet<FActorModifierAlignBetweenWeightedActor> WeightedActors = GetEnabledReferenceActors();

	float TotalWeight = 0.0f;
	for (const FActorModifierAlignBetweenWeightedActor& WeightedActor : WeightedActors)
	{
		if (WeightedActor.ActorWeak.IsValid())
		{
			TotalWeight += WeightedActor.Weight;	
		}
	}

	if (TotalWeight > 0.0f)
	{
		FVector AverageWeightedLocation = FVector::ZeroVector;
		
		for (const FActorModifierAlignBetweenWeightedActor& WeightedActor : WeightedActors)
		{
			if (const AActor* Actor = WeightedActor.ActorWeak.Get())
			{
				AverageWeightedLocation += Actor->GetActorLocation() * (WeightedActor.Weight / TotalWeight);
			}
		}
		
		ModifyActor->SetActorLocation(AverageWeightedLocation);
	}

	Next();
}

void UActorModifierAlignBetweenModifier::OnModifiedActorTransformed()
{
	Super::OnModifiedActorTransformed();

	MarkModifierDirty();
}

#if WITH_EDITOR
void UActorModifierAlignBetweenModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName ReferenceActorsPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAlignBetweenModifier, ReferenceActors);
	
	if (MemberName == ReferenceActorsPropertyName)
	{
		OnReferenceActorsChanged();
	}
}

void UActorModifierAlignBetweenModifier::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	OnReferenceActorsChanged();
	
	Super::PostTransacted(TransactionEvent);
}
#endif // WITH_EDITOR

TSet<AActor*> UActorModifierAlignBetweenModifier::GetActors(const bool bEnabledOnly) const
{
	TSet<AActor*> OutActors;

	for (const FActorModifierAlignBetweenWeightedActor& WeightedActor : ReferenceActors)
	{
		if (!bEnabledOnly|| (bEnabledOnly && WeightedActor.bEnabled))
		{
			OutActors.Add(WeightedActor.ActorWeak.Get());
		}
	}

	return OutActors;
}

void UActorModifierAlignBetweenModifier::SetReferenceActors(const TSet<FActorModifierAlignBetweenWeightedActor>& InReferenceActors)
{
	ReferenceActors = InReferenceActors;
	OnReferenceActorsChanged();
}

bool UActorModifierAlignBetweenModifier::AddReferenceActor(const FActorModifierAlignBetweenWeightedActor& InReferenceActor)
{
	const AActor* const ModifyActor = GetModifiedActor();
	if (!IsValid(ModifyActor))
	{
		return false;
	}

	if (!InReferenceActor.ActorWeak.IsValid() || InReferenceActor.ActorWeak == ModifyActor)
	{
		return false;
	}

	bool bAlreadyInSet = false;
	ReferenceActors.Add(InReferenceActor, &bAlreadyInSet);

	if (!bAlreadyInSet)
	{
		SetTransformExtensionReferenceActors();

		MarkModifierDirty();
	}

	return !bAlreadyInSet;
}

bool UActorModifierAlignBetweenModifier::RemoveReferenceActor(AActor* const InActor)
{
	if (!IsValid(InActor))
	{
		return false;
	}

	if (ReferenceActors.Remove(FActorModifierAlignBetweenWeightedActor(InActor)) > 0)
	{
		MarkModifierDirty();
		
		return true;
	}

	return false;
}

bool UActorModifierAlignBetweenModifier::FindReferenceActor(AActor* InActor, FActorModifierAlignBetweenWeightedActor& OutReferenceActor) const
{
	if (!IsValid(InActor))
	{
		return false;
	}

	if (const FActorModifierAlignBetweenWeightedActor* OutReference = ReferenceActors.Find(FActorModifierAlignBetweenWeightedActor(InActor)))
	{
		OutReferenceActor = *OutReference;
		return true;
	}

	return false;
}

void UActorModifierAlignBetweenModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("AlignBetween"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Align Between"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Positions an actor between a group of weighted actors"));
#endif
}

void UActorModifierAlignBetweenModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	// Check actor is reference actor enabled
	FActorModifierAlignBetweenWeightedActor ReferenceActor;
	FindReferenceActor(InActor, ReferenceActor);

	if (ReferenceActor.IsValid() && ReferenceActor.bEnabled && ReferenceActor.Weight > 0)
	{
		MarkModifierDirty();
	}
}

void UActorModifierAlignBetweenModifier::SetTransformExtensionReferenceActors()
{
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TSet<TWeakObjectPtr<AActor>> ExtensionActors;
		Algo::Transform(ReferenceActors, ExtensionActors, [](const FActorModifierAlignBetweenWeightedActor& InActor)->AActor*
		{
			return InActor.ActorWeak.Get();
		});
		TransformExtension->TrackActors(ExtensionActors, true);
	}
}

void UActorModifierAlignBetweenModifier::OnReferenceActorsChanged()
{
	// Make sure the modifying actor is not part of the array.
	AActor* const ModifyActor = GetModifiedActor();
	if (!IsValid(ModifyActor))
	{
		return;
	}
	
	ReferenceActors.Remove(FActorModifierAlignBetweenWeightedActor(ModifyActor));

	SetTransformExtensionReferenceActors();

	MarkModifierDirty();
}

TSet<FActorModifierAlignBetweenWeightedActor> UActorModifierAlignBetweenModifier::GetEnabledReferenceActors() const
{
	TSet<FActorModifierAlignBetweenWeightedActor> OutReferences;
	OutReferences.Reserve(ReferenceActors.Num());

	for (const FActorModifierAlignBetweenWeightedActor& WeightedActor : ReferenceActors)
	{
		if (WeightedActor.IsValid())
		{
			OutReferences.Add(WeightedActor);
		}
	}

	return OutReferences;
}

#undef LOCTEXT_NAMESPACE
