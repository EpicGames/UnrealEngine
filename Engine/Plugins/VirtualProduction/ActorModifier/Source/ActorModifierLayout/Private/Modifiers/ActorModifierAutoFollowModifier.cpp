// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierAutoFollowModifier.h"

#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "GameFramework/Actor.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Utilities/ActorModifierActorUtils.h"

#define LOCTEXT_NAMESPACE "ActorModifierAutoFollowModifier"

bool UActorModifierAutoFollowModifier::IsModifierDirtyable() const
{
	AActor* const ActorModified = GetModifiedActor();
	AActor* const FollowedActor = ReferenceActor.ReferenceActorWeak.Get();

	if (!IsValid(FollowedActor) || !IsValid(ActorModified))
	{
		return Super::IsModifierDirtyable();
	}

	const FBox ReferenceActorLocalBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(FollowedActor, true);
	const FBox ModifiedActorLocalBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(ActorModified, true);

	// Compare bounds/center to detect changes
	if (ReferenceActorLocalBounds.Equals(CachedReferenceBounds, 0.01)
		&& ModifiedActorLocalBounds.Equals(CachedModifiedBounds, 0.01))
	{
		return Super::IsModifierDirtyable();
	}

	return true;
}

void UActorModifierAutoFollowModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.AllowTick(true);
	InMetadata.SetName(TEXT("AutoFollow"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Auto Follow"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Positions an actor relative to another actor using their bounds"));
#endif
}

void UActorModifierAutoFollowModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierTransformUpdateExtension>(this);
	AddExtension<FActorModifierRenderStateUpdateExtension>(this);

	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->TrackSceneTree(0, &ReferenceActor);
	}

	bDeprecatedPropertiesMigrated = true;
}

void UActorModifierAutoFollowModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	// Save actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(true))
	{
		LayoutShared->SaveActorState(this, GetModifiedActor());
	}
}

void UActorModifierAutoFollowModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	// Restore actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(false))
	{
		LayoutShared->RestoreActorState(this, GetModifiedActor());
	}
}

void UActorModifierAutoFollowModifier::OnModifiedActorTransformed()
{
	const AActor* const FollowedActor = ReferenceActor.ReferenceActorWeak.Get();
	const AActor* const ActorModified = GetModifiedActor();

	if (!FollowedActor || !ActorModified)
	{
		return;
	}

	// Compare current location and previous followed location but only component of followed axis to allow movements
	if (UE::ActorModifier::ActorUtils::IsAxisVectorEquals(ActorModified->GetActorLocation(), CachedFollowLocation, FollowedAxis))
	{
		return;
	}

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::Apply()
{
	AActor* const ModifyActor = GetModifiedActor();

	AActor* const FollowedActor = ReferenceActor.ReferenceActorWeak.Get();
	if (!IsValid(FollowedActor))
	{
		Next();
		return;
	}

	if (ModifyActor->IsAttachedTo(FollowedActor))
	{
		Fail(LOCTEXT("InvalidReferenceActor", "Followed actor cannot be a parent of modified actor"));
		return;
	}

	const FVector FollowAxisVector = UE::ActorModifier::ActorUtils::GetVectorAxis(FollowedAxis);
	if (FollowAxisVector.IsNearlyZero())
	{
		Next();
		return;
	}

	// Get Padding to use
	const FVector DistancePadding = FMath::Lerp<FVector>(DefaultDistance, MaxDistance, Progress / 100.0f);

	// Get actors bounds
	CachedReferenceBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(FollowedActor, true);
	CachedModifiedBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(ModifyActor, true);

	// Check if bounds are valid
	const bool bReferenceActorZeroSizeBounds = CachedReferenceBounds.GetSize().IsNearlyZero();
	const bool bModifyActorZeroSizeBounds = CachedModifiedBounds.GetSize().IsNearlyZero();

	// Get actors location
	const FVector ReferenceActorLocation = FollowedActor->GetActorLocation();
	const FVector ModifyActorLocation = ModifyActor->GetActorLocation();

	// Get bounds center (pivot)
	const FVector ReferenceActorCenter = !bReferenceActorZeroSizeBounds ? CachedReferenceBounds.GetCenter() : ReferenceActorLocation;
	const FVector ModifierActorCenter = !bModifyActorZeroSizeBounds ? CachedModifiedBounds.GetCenter() : ModifyActorLocation;

	// Get bounds extents
	const FVector ReferenceActorExtent = CachedReferenceBounds.GetExtent();
	const FVector ModifierActorExtent = CachedModifiedBounds.GetExtent();

	// using world space extents so that the modified actor is moved also taking into account reference actor rotation
	const FVector ReferenceActorLocalOffset = ReferenceActorExtent * OffsetAxis;
	const FVector ModifierActorLocalOffset = ModifierActorExtent * OffsetAxis;

	// Use user alignments for followed and modify actors
    const FVector ReferenceActorBoundsOffset = FollowedAlignment.LocalBoundsOffset(FBox(-ReferenceActorExtent, ReferenceActorExtent));
    const FVector ModifierActorBoundsOffset = LocalAlignment.LocalBoundsOffset(FBox(-ModifierActorExtent, ModifierActorExtent));

	// this offset can be non-zero when the actor pivot and bounds origin do not coincide: we need to take this into account
	const FVector ReferenceActorPivotToBoundsOffset = ReferenceActorLocation - ReferenceActorCenter;
	const FVector ModifiedActorPivotToBoundsOffset = ModifyActorLocation - ModifierActorCenter;

	const FVector OffsetLocation =
		// Reference actor extent - reference actor alignment
		ReferenceActorLocalOffset - ReferenceActorBoundsOffset
		// Modified actor extent + modified actor alignment
		+ ModifierActorLocalOffset + ModifierActorBoundsOffset
		// Distance progress offset
		+ DistancePadding
		// finally removing any existing pivot to bounds offset
		+ (ModifiedActorPivotToBoundsOffset - ReferenceActorPivotToBoundsOffset);

	// target location needs to start from reference actor bounds location + proper location offset
	CachedFollowLocation = ModifyActorLocation + (ReferenceActorLocation - ModifyActorLocation + OffsetLocation) * FollowAxisVector;

	ModifyActor->SetActorLocation(CachedFollowLocation);

	Next();
}

void UActorModifierAutoFollowModifier::PostLoad()
{
	if (!bDeprecatedPropertiesMigrated
		&& ReferenceActor.ReferenceContainer == EActorModifierReferenceContainer::Other
		&& ReferenceActor.ReferenceActorWeak == nullptr)
	{
		ReferenceActor.ReferenceContainer = ReferenceContainer_DEPRECATED;
		ReferenceActor.ReferenceActorWeak = ReferenceActorWeak_DEPRECATED;
		ReferenceActor.bSkipHiddenActors = bIgnoreHiddenActors_DEPRECATED;

		bDeprecatedPropertiesMigrated = true;
	}

	Super::PostLoad();
}

#if WITH_EDITOR
void UActorModifierAutoFollowModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName ReferenceActorPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, ReferenceActor);
	static const FName DefaultDistancePropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, DefaultDistance);
	static const FName MaxDistancePropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, MaxDistance);
	static const FName ProgressPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, Progress);
	static const FName FollowerAlignmentPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, FollowedAlignment);
	static const FName LocalAlignmentPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, LocalAlignment);
	static const FName OffsetAxisPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, OffsetAxis);
	static const FName FollowedAxisPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierAutoFollowModifier, FollowedAxis);

	if (MemberName == ReferenceActorPropertyName)
	{
		OnReferenceActorChanged();
	}
	else if (MemberName == DefaultDistancePropertyName
		|| MemberName == MaxDistancePropertyName
		|| MemberName == ProgressPropertyName
		|| MemberName == FollowerAlignmentPropertyName
		|| MemberName == LocalAlignmentPropertyName
		|| MemberName == OffsetAxisPropertyName)
	{
		MarkModifierDirty();
	}
	else if (MemberName == FollowedAxisPropertyName)
	{
		OnFollowedAxisChanged();
	}
}

void UActorModifierAutoFollowModifier::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	OnReferenceActorChanged();

	Super::PostTransacted(TransactionEvent);
}
#endif // WITH_EDITOR

void UActorModifierAutoFollowModifier::SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor)
{
	if (ReferenceActor == InReferenceActor)
	{
		return;
	}

	ReferenceActor = InReferenceActor;
	OnReferenceActorChanged();
}

void UActorModifierAutoFollowModifier::SetFollowedAxis(int32 InFollowedAxis)
{
	if (FollowedAxis == InFollowedAxis)
	{
		return;
	}

	FollowedAxis = InFollowedAxis;
	OnFollowedAxisChanged();
}

void UActorModifierAutoFollowModifier::OnFollowedAxisChanged()
{
	const FVector FollowedAxisVector = UE::ActorModifier::ActorUtils::GetVectorAxis(FollowedAxis);

	LocalAlignment.bUseDepth = FollowedAlignment.bUseDepth = FollowedAxisVector.X != 0;
	LocalAlignment.bUseHorizontal = FollowedAlignment.bUseHorizontal = FollowedAxisVector.Y != 0;
	LocalAlignment.bUseVertical = FollowedAlignment.bUseVertical = FollowedAxisVector.Z != 0;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetDefaultDistance(const FVector& InDefaultDistance)
{
	DefaultDistance = InDefaultDistance;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetMaxDistance(const FVector& InMaxDistance)
{
	MaxDistance = InMaxDistance;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetProgress(const FVector& InProgress)
{
	Progress = InProgress;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetFollowedAlignment(const FActorModifierAnchorAlignment& InFollowedAlignment)
{
	FollowedAlignment = InFollowedAlignment;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetLocalAlignment(const FActorModifierAnchorAlignment& InLocalAlignment)
{
	LocalAlignment = InLocalAlignment;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::SetOffsetAxis(const FVector& InOffsetAxis)
{
	OffsetAxis = InOffsetAxis;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	const AActor* ActorModified = GetModifiedActor();
	if (!IsValid(ActorModified))
	{
		return;
	}

	const AActor* FollowedActor = ReferenceActor.ReferenceActorWeak.Get();
	const bool bIsAttachedToReferenceActor = InActor->IsAttachedTo(FollowedActor);
	const bool bIsReferenceActor = InActor == FollowedActor;

	if (!bInParentMoved && IsValid(FollowedActor) && (bIsAttachedToReferenceActor || bIsReferenceActor))
	{
		MarkModifierDirty();
	}
}

void UActorModifierAutoFollowModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	const AActor* ActorModified = GetModifiedActor();
	if (!IsValid(ActorModified))
	{
		return;
	}

	const AActor* FollowedActor = ReferenceActor.ReferenceActorWeak.Get();
	const UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(false);

	if (!InActor || !FollowedActor || !LayoutShared)
	{
		return;
	}

	const bool bIsReferenceActor = FollowedActor == InActor;
	const bool bIsAttachedToReferenceActor = InActor->IsAttachedTo(FollowedActor);

	if (!bIsReferenceActor && !bIsAttachedToReferenceActor)
	{
		return;
	}

	const bool bModifierDirtyable = IsModifierDirtyable();

	if (bModifierDirtyable)
	{
		MarkModifierDirty();
	}
}

void UActorModifierAutoFollowModifier::OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor)
{
	Super::OnSceneTreeTrackedActorChanged(InIdx, InPreviousActor, InNewActor);

	if (InNewActor == GetModifiedActor())
	{
		OnReferenceActorChanged();
		return;
	}

	// Untrack reference actor
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActor(InPreviousActor);
		TransformExtension->TrackActor(InNewActor, true);
	}

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx,
	const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	// Untrack reference actor children
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActors(InPreviousChildrenActors);
		TransformExtension->TrackActors(InNewChildrenActors, false);
	}

	ChildrenActorsWeak = InNewChildrenActors;

	MarkModifierDirty();
}

void UActorModifierAutoFollowModifier::OnReferenceActorChanged()
{
	const AActor* FollowerActor = GetModifiedActor();
	const AActor* TrackedActor = ReferenceActor.ReferenceActorWeak.Get();

	if (TrackedActor == FollowerActor
		|| FollowerActor->IsAttachedTo(TrackedActor))
	{
		ReferenceActor.ReferenceActorWeak = nullptr;
	}

	if (const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->CheckTrackedActorUpdate(0);
	}
}

#undef LOCTEXT_NAMESPACE
