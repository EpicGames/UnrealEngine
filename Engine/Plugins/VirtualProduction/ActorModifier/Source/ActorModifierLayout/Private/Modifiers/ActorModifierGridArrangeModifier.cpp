// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierGridArrangeModifier.h"

#include "GameFramework/Actor.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Shared/ActorModifierVisibilityShared.h"

#define LOCTEXT_NAMESPACE "ActorModifierGridArrangeModifier"

#if WITH_EDITOR
void UActorModifierGridArrangeModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName CountPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierGridArrangeModifier, Count);
	static const FName SpreadPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierGridArrangeModifier, Spread);
	static const FName StartCornerPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierGridArrangeModifier, StartCorner);
	static const FName StartDirectionPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierGridArrangeModifier, StartDirection);

	if (MemberName == CountPropertyName
		|| MemberName == SpreadPropertyName
		|| MemberName == StartCornerPropertyName
		|| MemberName == StartDirectionPropertyName)
	{
		MarkModifierDirty();
	}
}
#endif

void UActorModifierGridArrangeModifier::SetCount(const FIntPoint& InCount)
{
	const FIntPoint NewCount = InCount.ComponentMax(FIntPoint(1, 1));

	if (Count == NewCount)
	{
		return;
	}

	Count = NewCount;
	MarkModifierDirty();
}

void UActorModifierGridArrangeModifier::SetSpread(const FVector2D& InSpread)
{
	if (Spread == InSpread)
	{
		return;
	}

	Spread = InSpread;
	MarkModifierDirty();
}

void UActorModifierGridArrangeModifier::SetStartCorner(EActorModifierGridArrangeCorner2D InCorner)
{
	if (StartCorner == InCorner)
	{
		return;
	}

	StartCorner = InCorner;
	MarkModifierDirty();
}

void UActorModifierGridArrangeModifier::SetStartDirection(EActorModifierGridArrangeDirection InDirection)
{
	if (StartDirection == InDirection)
	{
		return;
	}

	StartDirection = InDirection;
	MarkModifierDirty();
}

void UActorModifierGridArrangeModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("GridArrange"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Grid Arrange"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Positions child actors in a 2D grid format"));
#endif
}

void UActorModifierGridArrangeModifier::Apply()
{
	AActor* const ModifyActor = GetModifiedActor();

	const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>();
	if (!SceneExtension)
	{
		Fail(LOCTEXT("InvalidSceneExtension", "Scene extension could not be found"));
		return;
	}

	if (Count.X < 1 || Count.Y < 1)
	{
		Fail(LOCTEXT("InvalidGridCount", "Count must be greater than 0"));
		return;
	}

	const TArray<TWeakObjectPtr<AActor>> AttachedActors = SceneExtension->GetDirectChildrenActor(ModifyActor);
	const int32 TotalSlotCount = Count.X * Count.Y;
	const int32 AttachedActorCount = AttachedActors.Num();

	auto GetGridX = [this](const int32 ChildIndex)
	{
		return StartDirection == EActorModifierGridArrangeDirection::Horizontal
			? ChildIndex % Count.X
			: ChildIndex / Count.X;
	};
	auto GetGridY = [this](const int32 ChildIndex)
	{
		return StartDirection == EActorModifierGridArrangeDirection::Horizontal
			? ChildIndex / Count.X
			: ChildIndex % Count.X;
	};

	auto GetReversedGridX = [this, GetGridX](const int32 ChildIndex) { return (Count.X - 1) - GetGridX(ChildIndex); };
	auto GetReversedGridY = [this, GetGridY](const int32 ChildIndex) { return (Count.Y - 1) - GetGridY(ChildIndex); };

	constexpr bool bCreate = true;
	UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(bCreate);
	UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(bCreate);

	if (!VisibilityShared || !LayoutShared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid modifier shared object retrieved"));
		return;
	}

	TSet<TWeakObjectPtr<AActor>> NewChildrenActorsWeak;
	for (int32 ChildIndex = 0; ChildIndex < AttachedActorCount; ++ChildIndex)
	{
		AActor* AttachedActor = AttachedActors[ChildIndex].Get();

		if (!AttachedActor)
		{
			continue;
		}

		{
			// Track all new children actors
			TArray<AActor*> ChildrenActors { AttachedActor };
			AttachedActor->GetAttachedActors(ChildrenActors, false, true);
			for (AActor* ChildActor : ChildrenActors)
			{
				NewChildrenActorsWeak.Add(ChildActor);
			}
		}

		// No need to handle nested children actor, only direct children, visibility will propagate
		if (AttachedActor->GetAttachParentActor() != ModifyActor)
		{
			continue;
		}

		// Track this actor visibility state
		const bool bIsVisible = ChildIndex < TotalSlotCount;
		VisibilityShared->SetActorVisibility(this, AttachedActor, !bIsVisible, true);

		FVector RelativeOffset = FVector::ZeroVector;
		switch (StartCorner)
		{
			case EActorModifierGridArrangeCorner2D::TopLeft:
				RelativeOffset.Y = GetGridX(ChildIndex);
				RelativeOffset.Z = GetGridY(ChildIndex);
			break;
			case EActorModifierGridArrangeCorner2D::TopRight:
				RelativeOffset.Y = -GetReversedGridX(ChildIndex);
				RelativeOffset.Z = GetGridY(ChildIndex);
			break;
			case EActorModifierGridArrangeCorner2D::BottomLeft:
				RelativeOffset.Y = GetGridX(ChildIndex);
				RelativeOffset.Z = -GetReversedGridY(ChildIndex);
			break;
			case EActorModifierGridArrangeCorner2D::BottomRight:
				RelativeOffset.Y = -GetReversedGridX(ChildIndex);
				RelativeOffset.Z = -GetReversedGridY(ChildIndex);
			break;
		}

		RelativeOffset.Y *= Spread.X;
		RelativeOffset.Z *= -Spread.Y;

		// Track this actor layout state
		LayoutShared->SaveActorState(this, AttachedActor);
		AttachedActor->SetActorRelativeLocation(RelativeOffset);
	}

	// Untrack previous actors that are not attached anymore
	const TSet<TWeakObjectPtr<AActor>> UntrackActors = ChildrenActorsWeak.Difference(NewChildrenActorsWeak);
	LayoutShared->RestoreActorsState(this, UntrackActors);
	VisibilityShared->RestoreActorsState(this, UntrackActors);

	ChildrenActorsWeak = NewChildrenActorsWeak;

	Next();
}

#undef LOCTEXT_NAMESPACE
