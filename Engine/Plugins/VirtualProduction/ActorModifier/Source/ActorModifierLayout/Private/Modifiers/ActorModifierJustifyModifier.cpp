// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierJustifyModifier.h"

#include "Components/ActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Utilities/ActorModifierActorUtils.h"

#define LOCTEXT_NAMESPACE "ActorModifierJustifyModifier"

void UActorModifierJustifyModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.AllowTick(true);
	InMetadata.SetName(TEXT("Justify"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Justify"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Aligns child actors according to the specified justification option, based on their bounding boxes"));
#endif
}

bool UActorModifierJustifyModifier::IsModifierDirtyable() const
{
	const AActor* ActorModified = GetModifiedActor();
	if (!IsValid(ActorModified))
	{
		return Super::IsModifierDirtyable();
	}

	constexpr bool bSkipHidden = true;
	constexpr bool bTransformBox = false;
	const FBox TrackedActorLocalBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(ChildrenActorsWeak, ActorModified->GetActorTransform(), bSkipHidden, bTransformBox);

	if (TrackedActorLocalBounds.Equals(CachedTrackedBounds, 0.01))
	{
		return Super::IsModifierDirtyable();
	}

	return true;
}

void UActorModifierJustifyModifier::OnModifiedActorTransformed()
{
	// Do nothing if we move, only justify children
}

void UActorModifierJustifyModifier::Apply()
{
	const AActor* const ActorModified = GetModifiedActor();

	// Gather newly attached children and save current state
	TSet<TWeakObjectPtr<AActor>> NewChildrenActorsWeak;
	GetChildrenActors(NewChildrenActorsWeak);

	// the children bounds need to be aligned to the modified actor
	constexpr bool bSkipHidden = true;
	constexpr bool bTransformBox = false;
	CachedTrackedBounds = UE::ActorModifier::ActorUtils::GetActorsBounds(NewChildrenActorsWeak, ActorModified->GetActorTransform(), bSkipHidden, bTransformBox);

	FVector BoundsCenter;
	FVector BoundsExtent;
	CachedTrackedBounds.GetCenterAndExtents(BoundsCenter, BoundsExtent);

	const FVector AlignmentOffset = GetAlignmentOffset(BoundsExtent);
	const FVector AnchorOffsetVector = GetAnchorOffset();

	// used to store the offset needed to constrain or un-constrain the justification axis, the modified actor and its children movements
	// constraints start set to the bounds center vector, and are changed below based on different combinations
	const FVector ConstraintVector = GetConstraintVector(BoundsCenter, FVector::ZeroVector);
	const FVector ChildLocationOffset = ConstraintVector + AlignmentOffset - AnchorOffsetVector;

	constexpr bool bCreate = true;
	UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(bCreate);

	if (!LayoutShared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid modifier shared object retrieved"));
		return;
	}

	// unregister children transform update callbacks, we don't want them firing while the modifier is updating
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActors(ChildrenActorsWeak);
	}

	// Lets track children actors visibility to receive callback
	if (FActorModifierRenderStateUpdateExtension* RenderExtension = GetExtension<FActorModifierRenderStateUpdateExtension>())
	{
		RenderExtension->SetTrackedActorsVisibility(NewChildrenActorsWeak);
	}
	
	// Update child actor position
	for (TWeakObjectPtr<AActor>& ChildActorWeak : NewChildrenActorsWeak)
	{
		if (AActor* const Child = ChildActorWeak.Get())
		{
			if (Child->GetAttachParentActor() != ActorModified)
			{
				continue;
			}

			LayoutShared->SaveActorState(this, Child);

			const FVector RelativeLocation = Child->GetRootComponent()->GetRelativeLocation();
			const FVector DesiredLocation = RelativeLocation - ChildLocationOffset;

			Child->SetActorRelativeLocation(DesiredLocation);
		}
	}
		
	// Untrack previous actors that are not attached anymore
	LayoutShared->RestoreActorsState(this, ChildrenActorsWeak.Difference(NewChildrenActorsWeak));

	ChildrenActorsWeak = NewChildrenActorsWeak;
	
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->TrackActors(ChildrenActorsWeak, true);
	}

	Next();
}

#if WITH_EDITOR
void UActorModifierJustifyModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	static const TSet<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, HorizontalAlignment),
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, VerticalAlignment),
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, DepthAlignment),
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, HorizontalAnchor),
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, VerticalAnchor),
		GET_MEMBER_NAME_CHECKED(UActorModifierJustifyModifier, DepthAnchor)
	};

	if (PropertyNames.Contains(PropertyName))
	{
		MarkModifierDirty();
	}
}
#endif

void UActorModifierJustifyModifier::SetHorizontalAlignment(EActorModifierJustifyHorizontal InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::SetVerticalAlignment(EActorModifierJustifyVertical InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::SetDepthAlignment(EActorModifierJustifyDepth InDepthAlignment)
{
	DepthAlignment = InDepthAlignment;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::SetHorizontalAnchor(const float InHorizontalAnchor)
{
	HorizontalAnchor = InHorizontalAnchor;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::SetVerticalAnchor(const float InVerticalAnchor)
{
	VerticalAnchor = InVerticalAnchor;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::SetDepthAnchor(const float InDepthAnchor)
{
	DepthAnchor = InDepthAnchor;

	MarkModifierDirty();
}

void UActorModifierJustifyModifier::OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx,
	const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	// Overwrite this from parent, don't do anything here, not needed for justify
}

void UActorModifierJustifyModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	Super::OnRenderStateUpdated(InActor, InComponent);

	const AActor* ActorModified = GetModifiedActor();
	if (!IsValid(ActorModified))
	{
		return;
	}

	// Only handle direct child of modified actor
	if (InActor->GetAttachParentActor() != ActorModified)
	{
		return;
	}

	// Is the modifier dirtyable
	const bool bModifierDirtyable = IsModifierDirtyable();
	
	if (bModifierDirtyable)
	{
		MarkModifierDirty();
	}
}

void UActorModifierJustifyModifier::OnActorVisibilityChanged(AActor* InActor)
{
	Super::OnActorVisibilityChanged(InActor);

	const AActor* ActorModified = GetModifiedActor();
	if (!IsValid(ActorModified))
	{
		return;
	}

	if (!InActor->IsAttachedTo(ActorModified))
	{
		return;
	}
	
	MarkModifierDirty();
}

void UActorModifierJustifyModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	Super::OnTransformUpdated(InActor, bInParentMoved);

	const AActor* ActorModified = GetModifiedActor();
	if (!ActorModified || !InActor)
	{
		return;
	}

	if (!InActor->IsAttachedTo(ActorModified))
	{
		return;
	}
	
	if (bInParentMoved)
	{
		return;
	}
	
	// if there's at least one justification constraint active, we need to rearrange all children based on their position update: let's update the modifier
	if (HasHorizontalAlignment() || HasVerticalAlignment() || HasDepthAlignment())
	{
		MarkModifierDirty();
	}
}

FVector UActorModifierJustifyModifier::MakeConstrainedAxisVector() const
{
	/* Generating a vector, the axes of which are:
	 * 0.0 --> the axis is unconstrained
	 * 1.0 --> the axis is constrained
	 */

	return FVector(static_cast<float>(HasDepthAlignment()),
		static_cast<float>(HasHorizontalAlignment()),
		static_cast<float>(HasVerticalAlignment()));
}

void UActorModifierJustifyModifier::GetChildrenActors(TSet<TWeakObjectPtr<AActor>>& OutChildren) const
{
	ForEachActor<AActor>([this, &OutChildren](AActor* InActor)->bool
	{
		if (IsValid(InActor))
		{
			OutChildren.Add(InActor);
		}
		return true;
	}, EActorModifierCoreLookup::AllChildren);
}

void UActorModifierJustifyModifier::GetTrackedActors(const TSet<TWeakObjectPtr<AActor>>& InChildrenActors, TArray<TWeakObjectPtr<const AActor>>& OutTrackedActors) const
{
	for (const TWeakObjectPtr<AActor>& ChildActorWeak : InChildrenActors)
	{
		const AActor* ChildActor = ChildActorWeak.Get();
		if (!IsValid(ChildActor))
		{
			continue;
		}

		// Only track visible actors, skip collapsed one
		if (!UE::ActorModifier::ActorUtils::IsActorVisible(ChildActor))
		{
			continue;
		}
		
		OutTrackedActors.Add(ChildActor);
	}
}

FVector UActorModifierJustifyModifier::GetConstraintVector(const FVector& InBoundsCenter, const FVector& InModifiedActorPosition) const
{
	const FVector BoundsCenterToModifiedActor = InBoundsCenter - InModifiedActorPosition;
	const FVector ConstrainedAxisVector = MakeConstrainedAxisVector();

	// The ConstrainedAxisVector is used to filter out unconstrained axes from the BoundsCenterToModifiedActor vector
	const FVector ConstraintVector = ConstrainedAxisVector * BoundsCenterToModifiedActor;
	
	return ConstraintVector;
}

bool UActorModifierJustifyModifier::HasDepthAlignment() const
{
	return DepthAlignment != EActorModifierJustifyDepth::None;
}

bool UActorModifierJustifyModifier::HasHorizontalAlignment() const
{
	return HorizontalAlignment != EActorModifierJustifyHorizontal::None;
}

bool UActorModifierJustifyModifier::HasVerticalAlignment() const
{
	return VerticalAlignment != EActorModifierJustifyVertical::None;
}

FVector UActorModifierJustifyModifier::GetAnchorOffset() const
{
	// used to add a custom anchor offset, independent from bounds size
	// note: some axes will be zeroed below, if their respective alignment is Off
	FVector AnchorOffsetVector = FVector(DepthAnchor, HorizontalAnchor, VerticalAnchor);

	// setup depth alignment and anchor offsets
	switch (DepthAlignment)
	{
		case EActorModifierJustifyDepth::None:
			AnchorOffsetVector.X = 0.0f; // ignore depth anchor reference
			break;
		default:
			break;
	}
	
	// setup horizontal alignment and anchor offsets
	switch (HorizontalAlignment)
	{				
		case EActorModifierJustifyHorizontal::None:
			AnchorOffsetVector.Y = 0.0f; // ignore horizontal anchor reference
			break;
		default:
			break;
	}

	// setup vertical alignment and anchor offsets
	switch (VerticalAlignment)
	{
		case EActorModifierJustifyVertical::None:
			AnchorOffsetVector.Z = 0.0f; // ignore vertical anchor reference
			break;
		default:
			break;
	}

	return AnchorOffsetVector;
}

FVector UActorModifierJustifyModifier::GetAlignmentOffset(const FVector& InExtent) const
{
	// will be filled with justification values, based on bounds size
	FVector AlignmentOffset = FVector::ZeroVector;

	// setup depth alignment and anchor offsets
	switch (DepthAlignment)
	{
		case EActorModifierJustifyDepth::None:
			AlignmentOffset.X = 0.0f;
			break;
				
		case EActorModifierJustifyDepth::Front:
			AlignmentOffset.X = InExtent.X;
			break;
				
		case EActorModifierJustifyDepth::Center:
			AlignmentOffset.X = 0.0f;
			break;
				
		case EActorModifierJustifyDepth::Back:
			AlignmentOffset.X = -InExtent.X;
			break;
	}
	
	// setup horizontal alignment and anchor offsets
	switch (HorizontalAlignment)
	{				
		case EActorModifierJustifyHorizontal::None:
			AlignmentOffset.Y = 0.0f;
			break;
			
		case EActorModifierJustifyHorizontal::Left:
			AlignmentOffset.Y = -InExtent.Y;
			break;
				
		case EActorModifierJustifyHorizontal::Center:
			AlignmentOffset.Y = 0.0f;
			break;
				
		case EActorModifierJustifyHorizontal::Right:
			AlignmentOffset.Y = InExtent.Y;
			break;
	}

	// setup vertical alignment and anchor offsets
	switch (VerticalAlignment)
	{
		case EActorModifierJustifyVertical::None:
			AlignmentOffset.Z = 0.0f;
			break;
				
		case EActorModifierJustifyVertical::Top:
			AlignmentOffset.Z = InExtent.Z;
			break;
			
		case EActorModifierJustifyVertical::Center:
			AlignmentOffset.Z = 0.0f;
			break;
			
		case EActorModifierJustifyVertical::Bottom:
			AlignmentOffset.Z = -InExtent.Z;
			break;
	}
	
	return AlignmentOffset;
}

#undef LOCTEXT_NAMESPACE
