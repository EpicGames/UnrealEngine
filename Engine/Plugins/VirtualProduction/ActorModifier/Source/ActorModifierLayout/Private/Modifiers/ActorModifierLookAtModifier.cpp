// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierLookAtModifier.h"

#include "ActorModifierTypes.h"
#include "GameFramework/Actor.h"
#include "Misc/ITransaction.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Utilities/ActorModifierActorUtils.h"

#define LOCTEXT_NAMESPACE "ActorModifierLookAtModifier"

void UActorModifierLookAtModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierTransformUpdateExtension>(this);

	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->TrackSceneTree(0, &ReferenceActor);
	}

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		OrientationAxis = EActorModifierAxis::X;
	}

	bDeprecatedPropertiesMigrated = true;
}

void UActorModifierLookAtModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);
	
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(/** CreateIfNone */true))
	{
		LayoutShared->SaveActorState(this, GetModifiedActor(), EActorModifierTransformSharedState::Rotation);
	}
}

void UActorModifierLookAtModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(/** CreateIfNone */false))
	{
		LayoutShared->RestoreActorState(this, GetModifiedActor(), EActorModifierTransformSharedState::Rotation);
	}
}

void UActorModifierLookAtModifier::Apply()
{
	AActor* const ModifyActor = GetModifiedActor();

	const AActor* LookActor = ReferenceActor.ReferenceActorWeak.Get();
	if (!IsValid(LookActor))
	{
		Next();
		return;
	}

	const FRotator NewRotation = UE::ActorModifier::ActorUtils::FindLookAtRotation(ModifyActor->GetActorLocation(), LookActor->GetActorLocation(), OrientationAxis, bFlipAxis);
	ModifyActor->SetActorRotation(NewRotation);

	Next();
}

void UActorModifierLookAtModifier::OnModifiedActorTransformed()
{
	MarkModifierDirty();
}

void UActorModifierLookAtModifier::PostLoad()
{
	if (!bDeprecatedPropertiesMigrated
		&& ReferenceActor.ReferenceContainer == EActorModifierReferenceContainer::Other
		&& ReferenceActor.ReferenceActorWeak == nullptr)
	{
		ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
		ReferenceActor.ReferenceActorWeak = ReferenceActorWeak;
		ReferenceActor.bSkipHiddenActors = false;

		bDeprecatedPropertiesMigrated = true;
	}

	if (OrientationAxis == EActorModifierAxis::None)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		OrientationAxis = static_cast<EActorModifierAxis>(1 << static_cast<int32>(Axis));

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	Super::PostLoad();
}

#if WITH_EDITOR
void UActorModifierLookAtModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName ReferenceActorPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierLookAtModifier, ReferenceActor);
	static const FName OrientationAxisPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierLookAtModifier, OrientationAxis);
	static const FName FlipAxisPropertyName = GET_MEMBER_NAME_CHECKED(UActorModifierLookAtModifier, bFlipAxis);

	if (MemberName == ReferenceActorPropertyName)
	{
		OnReferenceActorChanged();
	}
	else if (MemberName == OrientationAxisPropertyName
		|| MemberName == FlipAxisPropertyName)
	{
		MarkModifierDirty();
	}
}

void UActorModifierLookAtModifier::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	OnReferenceActorChanged();
	
	Super::PostTransacted(TransactionEvent);
}
#endif // WITH_EDITOR

void UActorModifierLookAtModifier::SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor)
{
	if (ReferenceActor == InReferenceActor)
	{
		return;
	}
	
	ReferenceActor = InReferenceActor;
	OnReferenceActorChanged();
}

void UActorModifierLookAtModifier::OnReferenceActorChanged()
{
	if (ReferenceActor.ReferenceActorWeak.Get() == GetModifiedActor())
	{
		ReferenceActor.ReferenceActorWeak = nullptr;
	}

	if (const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->CheckTrackedActorUpdate(0);
	}
}

void UActorModifierLookAtModifier::SetOrientationAxis(EActorModifierAxis InAxis)
{
	if (OrientationAxis == InAxis || OrientationAxis == EActorModifierAxis::None)
	{
		return;
	}

	OrientationAxis = InAxis;
	MarkModifierDirty();
}

void UActorModifierLookAtModifier::SetFlipAxis(const bool bInFlipAxis)
{
	if (bFlipAxis == bInFlipAxis)
	{
		return;
	}

	bFlipAxis = bInFlipAxis;
	MarkModifierDirty();
}

void UActorModifierLookAtModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("LookAt"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Look At"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Rotates an actor to face another actor"));
#endif
}

void UActorModifierLookAtModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	if (InActor && InActor == ReferenceActor.ReferenceActorWeak.Get())
	{
		MarkModifierDirty();
	}
}

void UActorModifierLookAtModifier::OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor)
{
	Super::OnSceneTreeTrackedActorChanged(InIdx, InPreviousActor, InNewActor);
	
	if (InNewActor == GetModifiedActor())
	{
		OnReferenceActorChanged();
		return;
	}

	// Untrack reference actor
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActor(InPreviousActor);
		TransformExtension->TrackActor(InNewActor, true);
	}

	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
