// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierRadialArrangeModifier.h"

#include "ActorModifierTypes.h"
#include "GameFramework/Actor.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Shared/ActorModifierVisibilityShared.h"
#include "Utilities/ActorModifierActorUtils.h"

#define LOCTEXT_NAMESPACE "ActorModifierRadialArrangeModifier"

void UActorModifierRadialArrangeModifier::PostLoad()
{
	Super::PostLoad();

	if (OrientationAxis == EActorModifierAxis::None)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		OrientationAxis = static_cast<EActorModifierAxis>(1 << static_cast<int32>(OrientAxis));

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#if WITH_EDITOR
void UActorModifierRadialArrangeModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const TSet<FName> PropertiesName =
	{
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, Count),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, Rings),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, InnerRadius),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, OuterRadius),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, StartAngle),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, EndAngle),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, Arrangement),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, bStartFromOuterRadius),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, bOrient),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, OrientationAxis),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, BaseOrientation),
		GET_MEMBER_NAME_CHECKED(UActorModifierRadialArrangeModifier, bFlipOrient)
	};

	if (PropertiesName.Contains(MemberName))
	{
		MarkModifierDirty();
	}
}
#endif // WITH_EDITOR

void UActorModifierRadialArrangeModifier::SetCount(const int32 InCount)
{
	if (Count == InCount)
	{
		return;
	}

	Count = InCount;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetRings(const int32 InRings)
{
	if (Rings == InRings)
	{
		return;
	}

	Rings = InRings;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetInnerRadius(const float InInnerRadius)
{
	if (FMath::IsNearlyEqual(InnerRadius, InInnerRadius))
	{
		return;
	}

	InnerRadius = InInnerRadius;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetOuterRadius(const float InOuterRadius)
{
	if (FMath::IsNearlyEqual(OuterRadius, InOuterRadius))
	{
		return;
	}

	OuterRadius = InOuterRadius;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetStartAngle(const float InStartAngle)
{
	if (FMath::IsNearlyEqual(StartAngle, InStartAngle))
	{
		return;
	}

	StartAngle = InStartAngle;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetEndAngle(const float InEndAngle)
{
	if (FMath::IsNearlyEqual(EndAngle, InEndAngle))
	{
		return;
	}

	EndAngle = InEndAngle;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetArrangement(const EActorModifierRadialArrangeMode InArrangement)
{
	if (Arrangement == InArrangement)
	{
		return;
	}

	Arrangement = InArrangement;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetStartFromOuterRadius(const bool bInStartFromOuterRadius)
{
	if (bStartFromOuterRadius == bInStartFromOuterRadius)
	{
		return;
	}

	bStartFromOuterRadius = bInStartFromOuterRadius;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetOrient(const bool bInOrient)
{
	if (bOrient == bInOrient)
	{
		return;
	}

	bOrient = bInOrient;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetOrientationAxis(EActorModifierAxis InAxis)
{
	if (OrientationAxis == InAxis)
	{
		return;
	}

	OrientationAxis = InAxis;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetBaseOrientation(const FRotator& InRotation)
{
	if (BaseOrientation.Equals(InRotation))
	{
		return;
	}

	BaseOrientation = InRotation;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::SetFlipOrient(const bool bInFlipOrient)
{
	if (bFlipOrient == bInFlipOrient)
	{
		return;
	}

	bFlipOrient = bInFlipOrient;
	MarkModifierDirty();
}

void UActorModifierRadialArrangeModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("RadialArrange"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Radial Arrange"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Positions child actors in a 2D radial format"));
#endif
}

void UActorModifierRadialArrangeModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		OrientationAxis = EActorModifierAxis::X;
	}
}

void UActorModifierRadialArrangeModifier::OnModifiedActorTransformed()
{
	// Overwrite parent class behaviour don't do anything when moved
	// Let user rotate the container and choose the wanted plane
}

void UActorModifierRadialArrangeModifier::Apply()
{
	AActor* ModifyActor = GetModifiedActor();

	const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>();
	if (!SceneExtension)
	{
		Fail(LOCTEXT("InvalidSceneExtension", "Scene extension could not be found"));
		return;
	}

	TArray<TWeakObjectPtr<AActor>> AttachedActors = SceneExtension->GetDirectChildrenActor(ModifyActor);
	const int32 AttachedActorCount = AttachedActors.Num();
	const int32 TotalSlotCount = Count == -1 ? AttachedActorCount : FMath::Min(AttachedActorCount, Count);

	// open distance in degrees where children will be placed
	const float AngleOpenDistance = EndAngle - StartAngle;
	const float RadiusDistance = Rings > 1 ? OuterRadius - InnerRadius : 0.0f;
	const float RadiusDistancePerRing = RadiusDistance / Rings;

	auto CalculateRelativeOffset = [this, RadiusDistancePerRing](float InAngleInDegrees, const int32 InRingIndex)->FVector
	{
		InAngleInDegrees = FRotator::NormalizeAxis(InAngleInDegrees);

		double SlotSin = 0.0f;
		double SlotCos = 0.0f;
		FMath::SinCos(&SlotSin, &SlotCos, FMath::DegreesToRadians(InAngleInDegrees));

		const float RingStartOffset = RadiusDistancePerRing * InRingIndex;
		const float ChildRadius = InnerRadius + RingStartOffset;

		return FVector(ChildRadius * SlotCos, ChildRadius * SlotSin, 0);
	};

	constexpr bool bCreate = true;
	UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(bCreate);
	UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(bCreate);

	if (!LayoutShared || !VisibilityShared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid modifier shared object retrieved"));
		return;
	}

	TSet<TWeakObjectPtr<AActor>> NewChildrenActorsWeak;
	for (int32 ChildIndex = 0; ChildIndex < AttachedActorCount; ++ChildIndex)
	{
		AActor* AttachedActor = AttachedActors[ChildIndex].Get();

		if (!AttachedActor)
		{
			continue;
		}

		{
			// Track all new children actors
			TArray<AActor*> ChildrenActors { AttachedActor };
			AttachedActor->GetAttachedActors(ChildrenActors, false, true);
			for (AActor* ChildActor : ChildrenActors)
			{
				NewChildrenActorsWeak.Add(ChildActor);
			}
		}

		// No need to handle nested children actor, only direct children, visibility will propagate
		if (AttachedActor->GetAttachParentActor() != ModifyActor)
		{
			continue;
		}

		// Track this actor visibility state
		const bool bIsVisible = ChildIndex < TotalSlotCount;
		VisibilityShared->SetActorVisibility(this, AttachedActor, !bIsVisible, true);

		const int32 ChildIndexToUse = ChildIndex;
		float RingAngleStep = 0.0f;
		int32 RingIndex = 0;
		int32 RingColumnIndex = 0;
		float SlotAngle = 0.0f;

		switch (Arrangement)
		{
			/**
			 * Each radial ring will contain the same number of elements.
			 * The space between elements in the outer rings will be greater than the inner rings.
			 */
			case EActorModifierRadialArrangeMode::Monospace:
			{
				const int32 ChildrenPerRing = FMath::Max(1, FMath::CeilToInt32((float)TotalSlotCount / Rings));

				RingAngleStep = ChildrenPerRing > 1 ? (AngleOpenDistance / (ChildrenPerRing - 1)) : 0.0f;

				RingColumnIndex = ChildIndexToUse % ChildrenPerRing;
				RingIndex = FMath::FloorToInt32((float)ChildIndexToUse / ChildrenPerRing);

				SlotAngle = StartAngle
					+ (RingAngleStep * RingColumnIndex)
					+ 90; // adding 90 degrees to make 0 degrees face up instead of right

				break;
			}
			/**
			 * All elements in all radial rings have the same spacing between them.
			 * The number of elements in the inner rings will be greater than the outer rings.
			 */
			 // @TODO: back engineer this Viz Artist arrangement mode
			case EActorModifierRadialArrangeMode::Equal:
			{
				//const float RingAngleStep = AngleOpenDistance / ChildrenPerRing;
				//RingAngleStep = (AngleOpenDistance * Rings) / TotalSlotCount;
				RingAngleStep = (AngleOpenDistance / TotalSlotCount) * Rings;

				const int32 ChildrenPerRing = FMath::Max(1, FMath::CeilToInt32((float)TotalSlotCount / Rings));

				RingColumnIndex = ChildIndexToUse % ChildrenPerRing;
				RingIndex = FMath::FloorToInt32((float)ChildIndexToUse / ChildrenPerRing);

				SlotAngle = StartAngle
					+ (RingAngleStep * RingColumnIndex)
					+ 90; // adding 90 degrees to make 0 degrees face up instead of right

				break;
			}
		}

		if (bStartFromOuterRadius)
		{
			RingIndex = Rings - (RingIndex - 1);
		}

		// Track this actor layout state
		LayoutShared->SaveActorState(this, AttachedActor, EActorModifierTransformSharedState::LocationRotation);

		const FVector RelativeOffset = CalculateRelativeOffset(SlotAngle, RingIndex);
		AttachedActor->SetActorRelativeLocation(RelativeOffset);

		if (bOrient)
		{
			const FVector EyePosition = RelativeOffset;
			const FVector TargetPosition = FVector::ZeroVector;

			FRotator NewRotation = BaseOrientation + UE::ActorModifier::ActorUtils::FindLookAtRotation(EyePosition, TargetPosition, OrientationAxis, bFlipOrient);

			AttachedActor->SetActorRelativeRotation(NewRotation);
		}
		// Restore original rotation
		else
		{
			LayoutShared->RestoreActorState(this, AttachedActor, EActorModifierTransformSharedState::Rotation);
		}
	}

	// Untrack previous actors that are not attached anymore
	const TSet<TWeakObjectPtr<AActor>> UntrackActors = ChildrenActorsWeak.Difference(NewChildrenActorsWeak);
	LayoutShared->RestoreActorsState(this, UntrackActors);
	VisibilityShared->RestoreActorsState(this, UntrackActors);

	ChildrenActorsWeak = NewChildrenActorsWeak;

	Next();
}

#undef LOCTEXT_NAMESPACE
