// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierSplinePathModifier.h"

#include "Components/SplineComponent.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Shared/ActorModifierTransformShared.h"

#define LOCTEXT_NAMESPACE "ActorModifierLayoutSplinePathModifier"

void UActorModifierSplinePathModifier::SetSplineActorWeak(TWeakObjectPtr<AActor> InActor)
{
	if (SplineActorWeak == InActor)
	{
		return;
	}

	SplineActorWeak = InActor;
	OnSplineActorWeakChanged();
}

void UActorModifierSplinePathModifier::SetSplineActor(AActor* InActor)
{
	SetSplineActorWeak(InActor);
}

void UActorModifierSplinePathModifier::SetSampleMode(EActorModifierLayoutSplinePathSampleMode InMode)
{
	if (SampleMode == InMode)
	{
		return;
	}
	
	SampleMode = InMode;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetProgress(float InProgress)
{
	InProgress = FMath::Max(InProgress, 0.f);
	if (FMath::IsNearlyEqual(Progress, InProgress))
	{
		return;
	}

	Progress = InProgress;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetDistance(float InDistance)
{
	InDistance = FMath::Max(InDistance, 0.f);
	if (FMath::IsNearlyEqual(Distance, InDistance))
	{
		return;
	}

	Distance = InDistance;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetTime(float InTime)
{
	InTime = FMath::Max(InTime, 0.f);
	if (FMath::IsNearlyEqual(Time, InTime))
	{
		return;
	}

	Time = InTime;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetPointIndex(int32 InIndex)
{
	InIndex = FMath::Max(InIndex, 0);
	if (PointIndex == InIndex)
	{
		return;
	}

	PointIndex = InIndex;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetOrient(bool bInOrient)
{
	if (bOrient == bInOrient)
	{
		return;
	}

	bOrient = bInOrient;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetBaseOrientation(const FRotator& InOrientation)
{
	if (BaseOrientation.Equals(InOrientation))
	{
		return;
	}

	BaseOrientation = InOrientation;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::SetScale(bool bInScale)
{
	if (bScale == bInScale)
	{
		return;
	}

	bScale = bInScale;
	OnSplineOptionsChanged();
}

#if WITH_EDITOR
FName UActorModifierSplinePathModifier::GetSplineActorWeakPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, SplineActorWeak);
}

const TActorModifierPropertyChangeDispatcher<UActorModifierSplinePathModifier> UActorModifierSplinePathModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, SplineActorWeak), &UActorModifierSplinePathModifier::OnSplineActorWeakChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, SampleMode), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, Progress), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, Distance), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, Time), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, PointIndex), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, bOrient), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, BaseOrientation), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UActorModifierSplinePathModifier, bScale), &UActorModifierSplinePathModifier::OnSplineOptionsChanged },
};

void UActorModifierSplinePathModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UActorModifierSplinePathModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	if (InActor == SplineActorWeak.Get())
	{
		OnSplineOptionsChanged();
	}
}

void UActorModifierSplinePathModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	if (InActor == SplineActorWeak.Get())
	{
		MarkModifierDirty();
	}
}

void UActorModifierSplinePathModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("SplinePath"));
	InMetadata.SetCategory(TEXT("Layout"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Spline Path"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Samples a spline for an actor to follow a path"));
#endif

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return IsValid(InActor) && IsValid(InActor->GetRootComponent());
	});
}

void UActorModifierSplinePathModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);
	
	AddExtension<FActorModifierRenderStateUpdateExtension>(this);
	AddExtension<FActorModifierTransformUpdateExtension>(this);
}

void UActorModifierSplinePathModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);
	
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(/** CreateIfNone */false))
	{
		LayoutShared->RestoreActorState(this, GetModifiedActor());
	}
}

void UActorModifierSplinePathModifier::Apply()
{
	AActor* ActorModified = GetModifiedActor();
	if (!ActorModified)
	{
		return;
	}

	UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(true);
	if (!LayoutShared)
	{
		Fail(LOCTEXT("InvalidTransformSharedObject", "Invalid transform shared object"));
		return;
	}

	const USplineComponent* SplineComponent = SplineComponentWeak.Get();
	if (!SplineComponent)
	{
		Next();
		return;
	}

	if (SplineComponent->GetSplineLength() <= 0.f
		|| SplineComponent->GetNumberOfSplinePoints() <= 0
		|| SplineComponent->Duration <= UE_SMALL_NUMBER)
	{
		Fail(LOCTEXT("InvalidSplineValues", "Invalid spline values"));
		return;
	}

	float SampleDistance = 0.f;

	if (SampleMode == EActorModifierLayoutSplinePathSampleMode::Distance)
	{
		if (Distance < 0)
		{
			SampleDistance = SplineComponent->GetSplineLength() - FMath::Abs(FMath::Fmod(Distance, SplineComponent->GetSplineLength()));
		}
		else
		{
			SampleDistance = FMath::Fmod(Distance, SplineComponent->GetSplineLength());
		}
	}
	else if (SampleMode == EActorModifierLayoutSplinePathSampleMode::Percentage)
	{
		if (Progress < 0)
		{
			SampleDistance = SplineComponent->GetSplineLength() - FMath::Abs(FMath::Fmod(SplineComponent->GetSplineLength() * Progress / 100.f, SplineComponent->GetSplineLength()));
		}
		else
		{
			SampleDistance = FMath::Fmod(SplineComponent->GetSplineLength() * Progress / 100.f, SplineComponent->GetSplineLength());
		}
	}
	else if (SampleMode == EActorModifierLayoutSplinePathSampleMode::Time)
	{
		if (Time < 0)
		{
			SampleDistance = SplineComponent->GetSplineLength() - FMath::Abs(FMath::Fmod(SplineComponent->GetSplineLength() * Time / SplineComponent->Duration, SplineComponent->GetSplineLength()));
		}
		else
		{
			SampleDistance = FMath::Fmod(SplineComponent->GetSplineLength() * Time / SplineComponent->Duration, SplineComponent->GetSplineLength());
		}
	}
	else if (SampleMode == EActorModifierLayoutSplinePathSampleMode::Point)
	{
		int32 SampleIndex = PointIndex;

		if (SampleIndex < 0)
		{
			SampleIndex = SplineComponent->GetNumberOfSplinePoints() - FMath::Modulo(FMath::Abs(SampleIndex), SplineComponent->GetNumberOfSplinePoints() + 1);
		}
		else
		{
			SampleIndex = FMath::Modulo(SampleIndex, SplineComponent->GetNumberOfSplinePoints());
		}

		if (SampleIndex < SplineComponent->GetNumberOfSplinePoints())
		{
			SampleDistance = SplineComponent->GetDistanceAlongSplineAtSplinePoint(SampleIndex);
		}
		else
		{
			SampleDistance = SplineComponent->GetSplineLength();
		}
	}

	SampleDistance = FMath::Max(SampleDistance, 0.f);
	
	if (SplineComponent->IsClosedLoop())
	{
		SampleDistance = FMath::Fmod(SampleDistance, SplineComponent->GetSplineLength());
	}

	LayoutShared->SaveActorState(this, ActorModified, EActorModifierTransformSharedState::Location);

	const FVector WorldLocation = SplineComponent->GetLocationAtDistanceAlongSpline(SampleDistance, ESplineCoordinateSpace::World);
	ActorModified->SetActorLocation(WorldLocation);

	if (bOrient)
	{
		LayoutShared->SaveActorState(this, ActorModified, EActorModifierTransformSharedState::Rotation);
		const FQuat WorldRotationQuat = SplineComponent->GetQuaternionAtDistanceAlongSpline(SampleDistance, ESplineCoordinateSpace::World) * BaseOrientation.Quaternion();
		ActorModified->SetActorRotation(WorldRotationQuat);
	}
	else
	{
		LayoutShared->RestoreActorState(this, ActorModified, EActorModifierTransformSharedState::Rotation);
	}

	if (bScale)
	{
		LayoutShared->SaveActorState(this, ActorModified, EActorModifierTransformSharedState::Scale);
		const FVector WorldScale = SplineComponent->GetComponentScale() * SplineComponent->GetScaleAtDistanceAlongSpline(SampleDistance);
		ActorModified->SetActorScale3D(WorldScale);
	}
	else
	{
		LayoutShared->RestoreActorState(this, ActorModified, EActorModifierTransformSharedState::Scale);
	}

	Next();
}

void UActorModifierSplinePathModifier::OnSplineActorWeakChanged()
{
	const AActor* SplineActor = SplineActorWeak.Get();

	if (!SplineActor)
	{
		return;
	}

	USplineComponent* SplineComponent = SplineActor->FindComponentByClass<USplineComponent>();

	// Don't allow actors without spline component
	if (!SplineComponent)
	{
		SplineActorWeak.Reset();
	}

	// Don't update if we already track this component
	if (SplineComponentWeak == SplineComponent)
	{
		return;
	}

	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		if (const USplineComponent* OldComponent = SplineComponentWeak.Get())
		{
			TransformExtension->UntrackActor(OldComponent->GetOwner());
		}

		if (SplineComponent)
		{
			TransformExtension->TrackActor(SplineComponent->GetOwner(), /** Reset */false);
		}
	}
	
	SplineComponentWeak = SplineComponent;
	OnSplineOptionsChanged();
}

void UActorModifierSplinePathModifier::OnSplineOptionsChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
