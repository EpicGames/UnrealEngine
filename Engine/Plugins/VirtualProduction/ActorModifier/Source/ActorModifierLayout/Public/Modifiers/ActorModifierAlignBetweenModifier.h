// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "ActorModifierAlignBetweenModifier.generated.h"

class AActor;

/** Represents an actor with a weight and an enabled state. */
USTRUCT(BlueprintType)
struct FActorModifierAlignBetweenWeightedActor
{
	GENERATED_BODY()

	FActorModifierAlignBetweenWeightedActor()
	{}

	explicit FActorModifierAlignBetweenWeightedActor(AActor* InActor)
		: ActorWeak(InActor)
	{}

	explicit FActorModifierAlignBetweenWeightedActor(AActor* InActor, float InWeight, bool bInEnabled)
		: ActorWeak(InActor)
		, Weight(InWeight)
		, bEnabled(bInEnabled)
	{}

	/** Returns true if the actor is valid and the state is enabled. */
	bool IsValid() const
	{
		return ActorWeak.IsValid() && bEnabled;
	}

	friend uint32 GetTypeHash(const FActorModifierAlignBetweenWeightedActor& InItem)
	{
		return GetTypeHash(InItem.ActorWeak);
	}

	bool operator==(const FActorModifierAlignBetweenWeightedActor& Other) const
	{
		return ActorWeak == Other.ActorWeak;
	}

	/** An actor that will effect the placement location. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Motion Design")
	TWeakObjectPtr<AActor> ActorWeak;

	/** How much effect this actor has on the placement location. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Motion Design", Meta = (ClampMin = "0.0", UIMin = "0.0"))
	float Weight = 0.0f;

	/** If true, will consider this weighted actor when calculating the placement location. */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Motion Design")
	bool bEnabled = false;
};

/**
 * Moves the modifying actor to the averaged location between an array of specified actors.
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierAlignBetweenModifier : public UActorModifierCoreBase
	, public IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

public:
	/** Gets all reference actors and their weights. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AlignBetween")
	TSet<FActorModifierAlignBetweenWeightedActor> GetReferenceActors() const
	{
		return ReferenceActors;
	}

	/** Sets all reference actors and their weights. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AlignBetween")
	ACTORMODIFIERLAYOUT_API void SetReferenceActors(const TSet<FActorModifierAlignBetweenWeightedActor>& InReferenceActors);

	/** Adds an actor to the reference list. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AlignBetween")
	ACTORMODIFIERLAYOUT_API bool AddReferenceActor(const FActorModifierAlignBetweenWeightedActor& InReferenceActor);

	/** Removes an actor from the reference list. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AlignBetween")
	ACTORMODIFIERLAYOUT_API bool RemoveReferenceActor(AActor* const InActor);

	/** Finds an actor in the reference list. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AlignBetween")
	ACTORMODIFIERLAYOUT_API bool FindReferenceActor(AActor* InActor, FActorModifierAlignBetweenWeightedActor& OutReferenceActor) const;

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void Apply() override;
	virtual void OnModifiedActorTransformed() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaTransformUpdatedExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdatedExtension

	void OnReferenceActorsChanged();
	void SetTransformExtensionReferenceActors();

	/** Gets all actors from their reference actor structs. */
	TSet<AActor*> GetActors(const bool bEnabledOnly = false) const;

	/** Returns all valid reference actors that enabled and have a weight greater than 0. */
	TSet<FActorModifierAlignBetweenWeightedActor> GetEnabledReferenceActors() const;

	/** Editable set of reference actors and weights used to calculate the average location for this actor */
	UPROPERTY(EditInstanceOnly, Setter="SetReferenceActors", Getter="GetReferenceActors", Category="AlignBetween", meta=(AllowPrivateAccess="true"))
	TSet<FActorModifierAlignBetweenWeightedActor> ReferenceActors;
};
