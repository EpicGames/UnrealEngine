// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierTypes.h"
#include "Components/SceneComponent.h"
#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Modifiers/ActorModifierAttachmentBaseModifier.h"
#include "ActorModifierAutoFollowModifier.generated.h"

class AActor;
class UActorComponent;

/**
 * Moves the modifying actor along with a specified actor relative to the specified actor's bounds.
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierAutoFollowModifier : public UActorModifierAttachmentBaseModifier
	, public IActorModifierTransformUpdateHandler
	, public IActorModifierRenderStateUpdateHandler
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	const FActorModifierSceneTreeActor& GetReferenceActor() const
	{
		return ReferenceActor;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetFollowedAxis(int32 InFollowedAxis);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	int32 GetFollowedAxis() const
	{
		return FollowedAxis;
	}

	/** Sets the distance from this actor to the followed actor. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetDefaultDistance(const FVector& InDefaultDistance);

	/** Gets the distance from this actor to the followed actor. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FVector GetDefaultDistance() const
	{
		return DefaultDistance;
	}

	/** Sets the maximum distance from this actor to the followed actor. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetMaxDistance(const FVector& InMaxDistance);

	/** Gets the maximum distance from this actor to the followed actor. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FVector GetMaxDistance() const
	{
		return MaxDistance;
	}

	/** Sets the percent % progress from the maximum distance to the default distance. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetProgress(const FVector& InProgress);

	/** Gets the percent % progress from the maximum distance to the default distance. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FVector GetProgress() const
	{
		return Progress;
	}

	/** Sets the alignment for the followed actor's center. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetFollowedAlignment(const FActorModifierAnchorAlignment& InFollowedAlignment);

	/** Gets the alignment for the followed actor's center. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FActorModifierAnchorAlignment GetFollowedAlignment() const
	{
		return FollowedAlignment;
	}

	/** Sets the alignment for this actor's center. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetLocalAlignment(const FActorModifierAnchorAlignment& InLocalAlignment);

	/** Gets the alignment for this actor's center. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FActorModifierAnchorAlignment GetLocalAlignment() const
	{
		return LocalAlignment;
	}

	/** Sets the axis direction to offset this actor from the followed actor's bounds. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoFollow")
	ACTORMODIFIERLAYOUT_API void SetOffsetAxis(const FVector& InOffsetAxis);

	/** Gets the axis direction to offset this actor from the followed actor's bounds. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoFollow")
	FVector GetOffsetAxis() const
	{
		return OffsetAxis;
	}

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual bool IsModifierDirtyable() const override;
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaTransformUpdatedExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdatedExtension

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override {}
	//~ End IAvaRenderStateUpdateExtension

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) override;
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	void OnReferenceActorChanged();

	void OnFollowedAxisChanged();

	UPROPERTY(EditInstanceOnly, Setter="SetReferenceActor", Getter="GetReferenceActor", Category="AutoFollow", meta=(ShowOnlyInnerProperties, AllowPrivateAccess="true"))
	FActorModifierSceneTreeActor ReferenceActor;

	/** The method for finding a reference actor based on it's position in the parent's hierarchy. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	EActorModifierReferenceContainer ReferenceContainer_DEPRECATED = EActorModifierReferenceContainer::Other;

	/** The actor being followed by the modifier. This is user selectable if the Reference Container is set to "Other". */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	TWeakObjectPtr<AActor> ReferenceActorWeak_DEPRECATED = nullptr;

	/** If true, will search for the next visible actor based on the selected reference container. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	bool bIgnoreHiddenActors_DEPRECATED = false;

	/** Which axis should we follow */
	UPROPERTY(EditInstanceOnly, Setter="SetFollowedAxis", Getter="GetFollowedAxis", Category="AutoFollow", meta=(Bitmask, BitmaskEnum="/Script/ActorModifier.EActorModifierAxis", AllowPrivateAccess="true"))
	int32 FollowedAxis = static_cast<int32>(
		EActorModifierAxis::Y |
		EActorModifierAxis::Z
	);

	/** Based on followed axis, the direction to offset this actor from the followed actor's bounds. */
	UPROPERTY(EditInstanceOnly, Setter="SetOffsetAxis", Getter="GetOffsetAxis", Interp, Category="AutoFollow", meta=(UIMin="-1.0", UIMax="1.0", AllowPrivateAccess="true"))
	FVector OffsetAxis = FVector(0, 1, 0);

	/** The alignment for the followed actor's center. */
	UPROPERTY(EditInstanceOnly, Setter="SetFollowedAlignment", Getter="GetFollowedAlignment", Category="AutoFollow", meta=(AllowPrivateAccess="true"))
	FActorModifierAnchorAlignment FollowedAlignment;

	/** The alignment for this actor's center. */
	UPROPERTY(EditInstanceOnly, Setter="SetLocalAlignment", Getter="GetLocalAlignment", Category="AutoFollow", meta=(AllowPrivateAccess="true"))
	FActorModifierAnchorAlignment LocalAlignment;

	/** The distance from this actor to the followed actor. */
	UPROPERTY(EditInstanceOnly, Setter="SetDefaultDistance", Getter="GetDefaultDistance", Interp, DisplayName="Start Padding", Category="AutoFollow", meta=(AllowPrivateAccess="true"))
	FVector DefaultDistance;

	/** The maximum distance from this actor to the followed actor. */
	UPROPERTY(EditInstanceOnly, Setter="SetMaxDistance", Getter="GetMaxDistance", Interp, DisplayName="End Padding", Category="AutoFollow", meta=(AllowPrivateAccess="true"))
	FVector MaxDistance;

	/** Percent % progress from the maximum distance to the default distance. */
	UPROPERTY(EditInstanceOnly, Setter="SetProgress", Getter="GetProgress", Interp, DisplayName="Padding Progress", Category="AutoFollow", meta=(ClampMin="0.0", UIMin="0.0", ClampMax="100.0", UIMax="100.0", AllowPrivateAccess="true"))
	FVector Progress;

private:
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FVector CachedFollowLocation = FVector::ZeroVector;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FBox CachedReferenceBounds = FBox(EForceInit::ForceInit);

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FBox CachedModifiedBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	bool bDeprecatedPropertiesMigrated = false;
};