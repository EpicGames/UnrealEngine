// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "ActorModifierGridArrangeModifier.generated.h"

/** Specifies a starting direction when laying out a grid arrangement. */
UENUM(BlueprintType)
enum class EActorModifierGridArrangeDirection : uint8
{
	Horizontal,
	Vertical
};

/** Specifies a corner for a 2D rectangle. */
UENUM(BlueprintType)
enum class EActorModifierGridArrangeCorner2D : uint8
{
	TopLeft,
	TopRight,
	BottomLeft,
	BottomRight
};

/**
 * Arranges child actors in a 2D grid format.
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierGridArrangeModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	/** Sets the width and height of the grid. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|GridArrange")
	ACTORMODIFIERLAYOUT_API void SetCount(const FIntPoint& InCount);

	/** Returns the width and height of the grid. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|GridArrange")
	FIntPoint GetCount() const
	{
		return Count;
	}

	/** Sets the distance between each horizontal and vertical child. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|GridArrange")
	ACTORMODIFIERLAYOUT_API void SetSpread(const FVector2D& InSpread);

	/** Returns the distance between each horizontal and vertical child. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|GridArrange")
	FVector2D GetSpread() const
	{
		return Spread;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|GridArrange")
	ACTORMODIFIERLAYOUT_API void SetStartCorner(EActorModifierGridArrangeCorner2D InCorner);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|GridArrange")
	EActorModifierGridArrangeCorner2D GetStartCorner() const
	{
		return StartCorner;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|GridArrange")
	ACTORMODIFIERLAYOUT_API void SetStartDirection(EActorModifierGridArrangeDirection InDirection);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|GridArrange")
	EActorModifierGridArrangeDirection GetStartDirection() const
	{
		return StartDirection;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	/** The width and height of the grid. */
	UPROPERTY(EditInstanceOnly, Setter="SetCount", Getter="GetCount", Category="GridArrange", Meta=(ClampMin="1", UIMin="1", AllowPrivateAccess="true"))
	FIntPoint Count = FIntPoint(1, 1);

	/** The distance between each horizontal and vertical child. */
	UPROPERTY(EditInstanceOnly, Setter="SetSpread", Getter="GetSpread", Interp, Category="GridArrange", meta=(AllowPrivateAccess="true"))
	FVector2D Spread = FVector2D::ZeroVector;

	/** The 2D corner from which to start the arrangement. */
	UPROPERTY(EditInstanceOnly, Setter="SetStartCorner", Getter="GetStartCorner", Category="GridArrange", meta=(AllowPrivateAccess="true"))
	EActorModifierGridArrangeCorner2D StartCorner;

	/** The direction from which to start the arrangement. */
	UPROPERTY(EditInstanceOnly, Setter="SetStartDirection", Getter="GetStartDirection", Category="GridArrange", meta=(AllowPrivateAccess="true"))
	EActorModifierGridArrangeDirection StartDirection;
};
