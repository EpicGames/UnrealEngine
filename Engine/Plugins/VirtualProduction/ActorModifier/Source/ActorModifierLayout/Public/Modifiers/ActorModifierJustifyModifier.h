// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "ActorModifierJustifyModifier.generated.h"

UENUM(BlueprintType)
enum class EActorModifierJustifyVertical : uint8
{
	None,
	Top,
	Center,
	Bottom,
};

UENUM(BlueprintType)
enum class EActorModifierJustifyHorizontal : uint8
{
	None,
	Left,
	Center,
	Right,
};

UENUM(BlueprintType)
enum class EActorModifierJustifyDepth : uint8
{
	None,
	Front,
	Center,
	Back
};

/**
 * Justify Modifier
 * 
 * Aligns child actors, based on their bounding box, according to the specified justification
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierJustifyModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetHorizontalAlignment(EActorModifierJustifyHorizontal InHorizontalAlignment);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	EActorModifierJustifyHorizontal GetHorizontalAlignment() const
	{
		return HorizontalAlignment;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetVerticalAlignment(EActorModifierJustifyVertical InVerticalAlignment);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	EActorModifierJustifyVertical GetVerticalAlignment() const
	{
		return VerticalAlignment;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetDepthAlignment(EActorModifierJustifyDepth InDepthAlignment);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	EActorModifierJustifyDepth GetDepthAlignment() const
	{
		return DepthAlignment;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetHorizontalAnchor(float InHorizontalAnchor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	float GetHorizontalAnchor() const
	{
		return HorizontalAnchor;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetVerticalAnchor(float InVerticalAnchor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	float GetVerticalAnchor() const
	{
		return VerticalAnchor;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Justify")
	ACTORMODIFIERLAYOUT_API void SetDepthAnchor(float InDepthAnchor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Justify")
	float GetDepthAnchor() const
	{
		return DepthAnchor;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual bool IsModifierDirtyable() const override;
	virtual void OnModifiedActorTransformed() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override;
	//~ End IAvaRenderStateUpdateExtension

	//~ Begin IAvaTransformUpdateExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdateExtension

	// Get tracked actors from children actors
	void GetChildrenActors(TSet<TWeakObjectPtr<AActor>>& OutChildren) const;
	void GetTrackedActors(const TSet<TWeakObjectPtr<AActor>>& InChildrenActors, TArray<TWeakObjectPtr<const AActor>>& OutTrackedActors) const;

	bool HasDepthAlignment() const;
	bool HasHorizontalAlignment() const;
	bool HasVerticalAlignment() const;

	FVector GetConstraintVector(const FVector& InBoundsCenter, const FVector& InModifiedActorPosition) const;
	FVector MakeConstrainedAxisVector() const;
	FVector GetAnchorOffset() const;
	FVector GetAlignmentOffset(const FVector& InExtent) const;

	UPROPERTY(EditInstanceOnly, Setter="SetHorizontalAlignment", Getter="GetHorizontalAlignment", Category="Justify", meta=(AllowPrivateAccess="true"))
	EActorModifierJustifyHorizontal HorizontalAlignment;

	UPROPERTY(EditInstanceOnly, Setter="SetVerticalAlignment", Getter="GetVerticalAlignment", Category="Justify", meta=(AllowPrivateAccess="true"))
	EActorModifierJustifyVertical VerticalAlignment;

	UPROPERTY(EditInstanceOnly, Setter="SetDepthAlignment", Getter="GetDepthAlignment", Category="Justify", meta=(AllowPrivateAccess="true"))
	EActorModifierJustifyDepth DepthAlignment;

	UPROPERTY(EditInstanceOnly, Setter="SetHorizontalAnchor", Getter="GetHorizontalAnchor", Interp, Category="Justify", meta=(EditCondition="HorizontalAlignment != EActorModifierJustifyHorizontal::None", EditConditionHides, AllowPrivateAccess="true"))
	float HorizontalAnchor;

	UPROPERTY(EditInstanceOnly, Setter="SetVerticalAnchor", Getter="GetVerticalAnchor", Interp, Category="Justify", meta=(EditCondition="VerticalAlignment != EActorModifierJustifyVertical::None", EditConditionHides, AllowPrivateAccess="true"))
	float VerticalAnchor;

	UPROPERTY(EditInstanceOnly, Setter="SetDepthAnchor", Getter="GetDepthAnchor", Interp, Category="Justify", meta=(EditCondition = "DepthAlignment != EActorModifierJustifyDepth::None", EditConditionHides, AllowPrivateAccess="true"))
	float DepthAnchor;

private:
	/** Cached actors bounds to detect a change in tick */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FBox CachedTrackedBounds = FBox(EForceInit::ForceInit);
};
