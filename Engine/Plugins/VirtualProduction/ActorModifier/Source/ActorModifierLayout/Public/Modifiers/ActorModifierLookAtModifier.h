// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierTypes.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Modifiers/ActorModifierAttachmentBaseModifier.h"
#include "ActorModifierLookAtModifier.generated.h"

class AActor;

/**
 * Rotates the modifying actor to point it's specified axis at another actor.
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierLookAtModifier : public UActorModifierAttachmentBaseModifier
	, public IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|LookAt")
	ACTORMODIFIERLAYOUT_API void SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|LookAt")
	const FActorModifierSceneTreeActor& GetReferenceActor() const
	{
		return ReferenceActor;
	}
	
	/** Sets the axis that will point towards the reference actor. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|LookAt")
	ACTORMODIFIERLAYOUT_API void SetOrientationAxis(EActorModifierAxis InAxis);

	/** Returns the axis that will point towards the reference actor. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|LookAt")
	EActorModifierAxis GetOrientationAxis() const
	{
		return OrientationAxis;
	}

	/** Sets the look-at direction to be flipped. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|LookAt")
	ACTORMODIFIERLAYOUT_API void SetFlipAxis(const bool bInFlipAxis);

	/** Returns true if flipping the look-at rotation axis. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|LookAt")
	bool GetFlipAxis() const
	{
		return bFlipAxis;
	}

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void Apply() override;
	virtual void OnModifiedActorTransformed() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaTransformUpdateExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdateExtension

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	void OnReferenceActorChanged();

	UPROPERTY(EditInstanceOnly, Setter="SetReferenceActor", Getter="GetReferenceActor", Category="LookAt", meta=(ShowOnlyInnerProperties, AllowPrivateAccess="true"))
	FActorModifierSceneTreeActor ReferenceActor;
	
	/** The axis to orient look at */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="LookAt", meta=(AllowPrivateAccess="true"))
	EActorModifierAxis OrientationAxis = EActorModifierAxis::None;

	/** If true, will flip the look-at direction. */
	UPROPERTY(EditInstanceOnly, Setter="SetFlipAxis", Getter="GetFlipAxis", Category="LookAt", meta=(AllowPrivateAccess="true"))
	bool bFlipAxis = false;

	/** The actor to look at. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	TWeakObjectPtr<AActor> ReferenceActorWeak;
	
	/** The axis that will point towards the reference actor. */
	UE_DEPRECATED(5.6, "Use OrientationAxis instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use OrientationAxis instead"))
	EActorModifierAlignment Axis;

	UPROPERTY()
	bool bDeprecatedPropertiesMigrated = false;
};
