// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorModifierTypes.h"
#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "ActorModifierRadialArrangeModifier.generated.h"

/** Specifies how child elements will be arranged radially. */
UENUM(BlueprintType)
enum class EActorModifierRadialArrangeMode : uint8
{
	/**
	 * Each radial ring will contain the same number of elements.
	 * The space between elements in the outer rings will be greater than the inner rings.
	 */
	Monospace,
	/**
	 * All elements in all radial rings have the same spacing between them.
	 * The number of elements in the inner rings will be greater than the outer rings.
	 */
	 Equal
};

/** Enumerates how to layout the ring */
UENUM(BlueprintType)
enum class EActorModifierRadialArrangePlane : uint8
{
	XY,
	YZ,
	XZ,
};

/**
 * Arranges child actors in a circular rings around its center
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierRadialArrangeModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	/** Sets the number of child elements to use in the arrangement. Children whose index is greater than or equal to this value will be hidden. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetCount(const int32 InCount);

	/** Gets the number of child elements to use in the arrangement. Children whose index is greater than or equal to this value will be hidden. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	int32 GetCount() const
	{
		return Count;
	}

	/** Sets the number of rings. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetRings(const int32 InRings);

	/** Gets the number of rings. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	int32 GetRings() const
	{
		return Rings;
	}

	/** Sets the radius from the center to the first inner ring. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetInnerRadius(const float InInnerRadius);

	/** Gets the radius from the center to the first inner ring. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	float GetInnerRadius() const
	{
		return InnerRadius;
	}

	/** Sets the radius from the center to the last outer ring. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetOuterRadius(const float InOuterRadius);

	/** Gets the radius from the center to the last outer ring. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	float GetOuterRadius() const
	{
		return OuterRadius;
	}

	/** Sets the start angle for the arrangement space and moving clockwise. 0 = Up, -90 = Left, 90 = Right */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetStartAngle(const float InStartAngle);

	/** Gets the start angle for the arrangement space and moving clockwise. 0 = Up, -90 = Left, 90 = Right */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	float GetStartAngle() const
	{
		return StartAngle;
	}

	/** Sets the end angle for the arrangement space. 0 = Up, -90 = Left, 90 = Right */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetEndAngle(const float InEndAngle);

	/** Gets the end angle for the arrangement space. 0 = Up, -90 = Left, 90 = Right */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	float GetEndAngle() const
	{
		return EndAngle;
	}

	/** Defines how to arrange the child elements around the center. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetArrangement(const EActorModifierRadialArrangeMode InArrangement);

	/** Defines how to arrange the child elements around the center. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	EActorModifierRadialArrangeMode GetArrangement() const
	{
		return Arrangement;
	}

	/** If true, will arrange the child elements starting from the outer radius and moving to the inner radius. Has no effect if only using one ring. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetStartFromOuterRadius(const bool bInStartFromOuterRadius);

	/** If true, will arrange the child elements starting from the outer radius and moving to the inner radius. Has no effect if only using one ring. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	bool GetStartFromOuterRadius() const
	{
		return bStartFromOuterRadius;
	}

	/** If true, will orient the selected axis torwards the center. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetOrient(const bool bInOrient);

	/** If true, will orient the selected axis torwards the center. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	bool GetOrient() const
	{
		return bOrient;
	}

	/** Sets the axis to look at the center. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetOrientationAxis(EActorModifierAxis InAxis);

	/** Gets the axis to look at the center. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	EActorModifierAxis GetOrientationAxis() const
	{
		return OrientationAxis;
	}

	/** Sets the base rotation */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetBaseOrientation(const FRotator& InRotation);

	/** Gets the base rotation */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	const FRotator& GetBaseOrientation() const
	{
		return BaseOrientation;
	}

	/** If true, will flip the center orientation to face outwards. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|RadialArrange")
	ACTORMODIFIERLAYOUT_API void SetFlipOrient(const bool bInFlipOrient);

	/** If true, will flip the center orientation to face outwards. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|RadialArrange")
	bool GetFlipOrient() const
	{
		return bFlipOrient;
	}

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	/** The number of child elements to limit in the arrangement, or -1 if unlimited. Children whose index is greater than or equal to this value will be hidden. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="RadialArrange", meta=(ClampMin="-1", UIMin="-1", AllowPrivateAccess="true"))
	int32 Count = -1;

	/** The number of rings. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="RadialArrange", meta=(ClampMin="1", UIMin="1", AllowPrivateAccess="true"))
	int32 Rings = 1;

	/** The radius from the center to the first inner ring. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="RadialArrange", meta=(ClampMin="0.0", UIMin="0.0", AllowPrivateAccess="true"))
	float InnerRadius = 70.0f;

	/** The radius from the center to the last outer ring. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="RadialArrange", meta=(EditCondition="Rings > 1", EditConditionHides, ClampMin="0.0", UIMin="0.0", AllowPrivateAccess="true"))
	float OuterRadius = 200.0f;

	/** The start angle for the arrangement space and moving clockwise. 0 = Up, -90 = Left, 90 = Right */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="RadialArrange", meta=(ClampMin="-180.0", UIMin="-180.0", ClampMax="180.0", UIMax="180.0", AllowPrivateAccess="true"))
	float StartAngle = -180.0f;

	/** The end angle for the arrangement space. 0 = Up, -90 = Left, 90 = Right */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="RadialArrange", meta=(ClampMin = "-180.0", UIMin = "-180.0", ClampMax = "180.0", UIMax = "180.0", AllowPrivateAccess="true"))
	float EndAngle = 180.0f;

	/** Defines how to arrange the child elements around the center. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="RadialArrange", meta=(AllowPrivateAccess="true"))
	EActorModifierRadialArrangeMode Arrangement = EActorModifierRadialArrangeMode::Equal;

	/** If true, will arrange the child elements starting from the outer radius and moving to the inner radius. Has no effect if only using one ring. */
	UPROPERTY(EditInstanceOnly, Setter="SetStartFromOuterRadius", Getter="GetStartFromOuterRadius", Category="RadialArrange", meta=(AllowPrivateAccess="true"))
	bool bStartFromOuterRadius = false;

	/** If true, will orient the selected axis torwards the center. */
	UPROPERTY(EditInstanceOnly, Setter="SetOrient", Getter="GetOrient", Category="RadialArrange", meta=(AllowPrivateAccess="true"))
	bool bOrient = false;

	/** The axis to look at the center. */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="RadialArrange", meta=(EditCondition="bOrient", EditConditionHides, AllowPrivateAccess="true"))
	EActorModifierAxis OrientationAxis = EActorModifierAxis::None;

	/** Base rotation added on top of the orientation rotation computed */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="RadialArrange", meta=(EditCondition="bOrient", EditConditionHides, AllowPrivateAccess="true"))
	FRotator BaseOrientation = FRotator::ZeroRotator;

	UE_DEPRECATED(5.5, "Use OrientationAxis instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use OrientationAxis instead"))
	EActorModifierAlignment OrientAxis;

	/** If true, will flip the orientation axis to the opposite direction. */
	UPROPERTY(EditInstanceOnly, Setter="SetFlipOrient", Getter="GetFlipOrient", Category="RadialArrange", meta=(EditCondition="bOrient", EditConditionHides, AllowPrivateAccess="true"))
	bool bFlipOrient = false;
};
