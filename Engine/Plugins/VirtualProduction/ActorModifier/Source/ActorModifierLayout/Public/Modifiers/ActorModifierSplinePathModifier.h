// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Utilities/ActorModifierPropertyChangeDispatcher.h"
#include "ActorModifierSplinePathModifier.generated.h"

class USplineComponent;

UENUM(BlueprintType)
enum class EActorModifierLayoutSplinePathSampleMode : uint8
{
	Percentage,
	Distance,
	Time,
	Point
};

/**
 * This modifier allows to sample a spline and update the actor transform based on the spline path
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierSplinePathModifier : public UActorModifierCoreBase
	, public IActorModifierRenderStateUpdateHandler
	, public IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	ACTORMODIFIERLAYOUT_API static FName GetSplineActorWeakPropertyName();
#endif
	
	void SetSplineActorWeak(TWeakObjectPtr<AActor> InActor);
	TWeakObjectPtr<AActor> GetSplineActorWeak() const
	{
		return SplineActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetSplineActor(AActor* InActor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	AActor* GetSplineActor() const
	{
		return SplineActorWeak.Get();
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetSampleMode(EActorModifierLayoutSplinePathSampleMode InMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	EActorModifierLayoutSplinePathSampleMode GetSampleMode() const
	{
		return SampleMode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetProgress(float InProgress);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	float GetProgress() const
	{
		return Progress;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetDistance(float InDistance);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	float GetDistance() const
	{
		return Distance;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetTime(float InTime);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	float GetTime() const
	{
		return Time;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetPointIndex(int32 InIndex);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	int32 GetPointIndex() const
	{
		return PointIndex;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetOrient(bool bInOrient);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	bool GetOrient() const
	{
		return bOrient;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetBaseOrientation(const FRotator& InOrientation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	const FRotator& GetBaseOrientation() const
	{
		return BaseOrientation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplinePath")
	void SetScale(bool bInScale);
	
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplinePath")
	bool GetScale() const
	{
		return bScale;
	}
	
protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin IActorModifierRenderStateUpdateHandler
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override {}
	//~ End IActorModifierRenderStateUpdateHandler

	//~ Begin IActorModifierTransformUpdateHandler
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IActorModifierTransformUpdateHandler

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnSplineActorWeakChanged();
	void OnSplineOptionsChanged();
	
	/** Spline actor to retrieve the USplineComponent from */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(FilterActorByComponentClass="/Script/Engine.SplineComponent", AllowPrivateAccess="true"))
	TWeakObjectPtr<AActor> SplineActorWeak;

	/** How to sample the spline */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(AllowPrivateAccess="true"))
	EActorModifierLayoutSplinePathSampleMode SampleMode = EActorModifierLayoutSplinePathSampleMode::Percentage;
	
	/** Percentage progress to sample the spline at */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(EditCondition="SampleMode == EActorModifierLayoutSplinePathSampleMode::Percentage", EditConditionHides, Units=Percent, AllowPrivateAccess="true"))
	float Progress = 0.f;

	/** Distance to sample the spline at */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(EditCondition="SampleMode == EActorModifierLayoutSplinePathSampleMode::Distance", EditConditionHides, Units=Centimeters, AllowPrivateAccess="true"))
	float Distance = 0.f;

	/** Time to sample the spline at */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(EditCondition="SampleMode == EActorModifierLayoutSplinePathSampleMode::Time", EditConditionHides, AllowPrivateAccess="true"))
	float Time = 0.f;
	
	/** Point index to sample the spline at */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(EditCondition="SampleMode == EActorModifierLayoutSplinePathSampleMode::Point", EditConditionHides, AllowPrivateAccess="true"))
	int32 PointIndex = 0;

	/** Orient actor based on spline tangent */
	UPROPERTY(EditInstanceOnly, Setter="SetOrient", Getter="GetOrient", Category="SplinePath", meta=(AllowPrivateAccess="true"))
	bool bOrient = true;

	/** Base rotation added on top of the orientation rotation computed */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="SplinePath", meta=(EditCondition="bOrient", EditConditionHides, AllowPrivateAccess="true"))
	FRotator BaseOrientation = FRotator::ZeroRotator;

	/** Apply scale based on spline point scale */
	UPROPERTY(EditInstanceOnly, Setter="SetScale", Getter="GetScale", Category="SplinePath", meta=(AllowPrivateAccess="true"))
	bool bScale = false;

	/** Spline component found on the spline actor */
	UPROPERTY()
	TWeakObjectPtr<USplineComponent> SplineComponentWeak;

private:
#if WITH_EDITOR
	static const TActorModifierPropertyChangeDispatcher<UActorModifierSplinePathModifier> PropertyChangeDispatcher;
#endif
};