// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorModifierRendering : ModuleRules
{
    public ActorModifierRendering(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
	            "ActorModifier",
	            "ActorModifierCore",
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "CompositeCore",
                "CoreUObject",
                "Engine"
            }
        );
    }
}