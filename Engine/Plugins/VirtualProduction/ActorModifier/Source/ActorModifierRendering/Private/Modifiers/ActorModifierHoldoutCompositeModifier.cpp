// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/ActorModifierHoldoutCompositeModifier.h"
#include "Components/PrimitiveComponent.h"
#include "CompositeCoreSubsystem.h"
#include "HoldoutCompositeComponent.h"

#define LOCTEXT_NAMESPACE "ActorModifierHoldoutModifier"

void UActorModifierHoldoutCompositeModifier::SetIncludeChildren(bool bInInclude)
{
	if (bInInclude == bIncludeChildren)
	{
		return;
	}

	bIncludeChildren = bInInclude;
	OnIncludeChildrenChanged();
}

#if WITH_EDITOR
void UActorModifierHoldoutCompositeModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UActorModifierHoldoutCompositeModifier, bIncludeChildren))
	{
		OnIncludeChildrenChanged();
	}
}
#endif

void UActorModifierHoldoutCompositeModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("HoldoutComposite"));
	InMetadata.SetCategory(TEXT("Rendering"));
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor->FindComponentByClass<UHoldoutCompositeComponent>() == nullptr;
	});
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Holdout composite"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Renders the attached primitives in a separate pass and composites it back using an alpha holdout mask"));
#endif
}

void UActorModifierHoldoutCompositeModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	// Track hierarchy changes
	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
		ReferenceActor.ReferenceActorWeak = GetModifiedActor();
		ReferenceActor.bSkipHiddenActors = false;
		
		SceneExtension->TrackSceneTree(0, &ReferenceActor);
	}
}

void UActorModifierHoldoutCompositeModifier::RestorePreState()
{
	Super::RestorePreState();

	UnregisterPrimitiveComponents();
}

void UActorModifierHoldoutCompositeModifier::Apply()
{
	UCompositeCoreSubsystem* CompositeSubsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(GetWorld());
	if (!CompositeSubsystem)
	{
		Fail(LOCTEXT("InvalidCompositeSubsystem", "Composite subsystem is not valid"));
		return;
	}

	if (!UCompositeCoreSubsystem::IsProjectSettingsValid())
	{
		Fail(LOCTEXT("InvalidProjectSettings", "Invalid holdout rendering project settings"));
		return;
	}

	// Scan primitive components
	ForEachComponent<UPrimitiveComponent>(
		[this](UPrimitiveComponent* InComponent)->bool
		{
			PrimitiveComponentsWeak.Add(InComponent);
			return true;
		}
		, EActorModifierCoreComponentType::All
		, bIncludeChildren ? EActorModifierCoreLookup::SelfAndAllChildren : EActorModifierCoreLookup::Self
	);

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	PrimitiveComponents.Reserve(PrimitiveComponentsWeak.Num());

	Algo::TransformIf(
		PrimitiveComponentsWeak,
		PrimitiveComponents,
		[](const TWeakObjectPtr<UPrimitiveComponent>& InComponentWeak)
		{
			return InComponentWeak.IsValid();
		},
		[](const TWeakObjectPtr<UPrimitiveComponent>& InComponentWeak)
		{
			return InComponentWeak.Get();
		}
	);

	CompositeSubsystem->RegisterPrimitives(PrimitiveComponents);

	Next();
}

void UActorModifierHoldoutCompositeModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	if (bIncludeChildren)
	{
		MarkModifierDirty();
	}
}

void UActorModifierHoldoutCompositeModifier::OnIncludeChildrenChanged()
{
	MarkModifierDirty();
}

void UActorModifierHoldoutCompositeModifier::UnregisterPrimitiveComponents()
{
	if (UCompositeCoreSubsystem* CompositeSubsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(GetWorld()))
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		Algo::TransformIf(
			PrimitiveComponentsWeak,
			PrimitiveComponents,
			[](const TWeakObjectPtr<UPrimitiveComponent>& InComponentWeak)
			{
				return InComponentWeak.IsValid();
			},
			[](const TWeakObjectPtr<UPrimitiveComponent>& InComponentWeak)
			{
				return InComponentWeak.Get();
			}
		);

		CompositeSubsystem->UnregisterPrimitives(PrimitiveComponents);

		PrimitiveComponentsWeak.Empty();
	}
}

#undef LOCTEXT_NAMESPACE
