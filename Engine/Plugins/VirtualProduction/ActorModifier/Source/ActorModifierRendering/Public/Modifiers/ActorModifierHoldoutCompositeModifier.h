// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Modifiers/ActorModifierAttachmentBaseModifier.h"
#include "ActorModifierHoldoutCompositeModifier.generated.h"

class AActor;
class UPrimitiveComponent;

/**
 * Track primitive components to render them in a separate pass and composites it back using an alpha holdout mask
 */
UCLASS(MinimalAPI, BlueprintType)
class UActorModifierHoldoutCompositeModifier : public UActorModifierAttachmentBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Holdout")
	void SetIncludeChildren(bool bInInclude);

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Holdout")
	bool GetIncludeChildren() const
	{
		return bIncludeChildren;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void RestorePreState() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	void OnIncludeChildrenChanged();
	void UnregisterPrimitiveComponents();

	/** If true, will apply itself onto children actors as well */
	UPROPERTY(EditInstanceOnly, Setter="SetIncludeChildren", Getter="GetIncludeChildren", Category="HoldoutComposite", meta=(AllowPrivateAccess="true"))
	bool bIncludeChildren = false;

	/** Primitive components tracked by the composite subsystem */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPrimitiveComponent>> PrimitiveComponentsWeak;

	/** Used to track self modified actor for changes */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FActorModifierSceneTreeActor ReferenceActor;
};
